// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class DerivedDataTool : ModuleRules
{
	public DerivedDataTool(ReadOnlyTargetRules Target) : base(Target)
	{
		bTreatAsEngineModule = true;

		PublicIncludePathModuleNames.Add("Launch");

		PrivateDependencyModuleNames.Add("ApplicationCore");
		PrivateDependencyModuleNames.Add("Core");
		PrivateDependencyModuleNames.Add("DerivedDataCache");
		PrivateDependencyModuleNames.Add("DesktopPlatform");
		PrivateDependencyModuleNames.Add("Projects");

		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;
	}
}
