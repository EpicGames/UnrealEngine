// Copyright Epic Games, Inc. All Rights Reserved.

#include "DerivedDataToolCommand.h"

#include "Algo/Accumulate.h"
#include "Algo/AllOf.h"
#include "Async/EventCount.h"
#include "Async/Mutex.h"
#include "Async/ParallelFor.h"
#include "Async/SharedLock.h"
#include "Async/SharedMutex.h"
#include "Async/UniqueLock.h"
#include "Concepts/Integral.h"
#include "Containers/ConcurrentChunkedSparseArray.h"
#include "Containers/Map.h"
#include "Containers/ProbingHashTable.h"
#include "DerivedDataCache.h"
#include "DerivedDataCacheKeyFilter.h"
#include "DerivedDataCacheReplay.h"
#include "DerivedDataCacheTraits.h"
#include "DerivedDataConfig.h"
#include "DerivedDataRequestOwner.h"
#include "DerivedDataSerialization.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformTime.h"
#include "Logging/LogMacros.h"
#include "Logging/StructuredLog.h"
#include "Misc/AlignedElement.h"
#include "Misc/Parse.h"
#include "Misc/ScopeExit.h"
#include "Misc/StringBuilder.h"
#include "Templates/FunctionWithContext.h"

namespace UE::DerivedData::Tool
{

/** Minimum period between progress reports. */
constexpr static FMonotonicTimeSpan ProgressPeriod = FMonotonicTimeSpan::FromSeconds(10.0);

constexpr static int32 MaxActiveScanRequestBatchCount = 256;
constexpr static int32 ScanRequestBatchSize = 16;

constexpr static int32 MaxActiveGetRequestBatchCount = 64;
constexpr static int32 GetRequestBatchSize = 16;

constexpr static int32 MaxActivePutRequestBatchCount = 64;
constexpr static int32 PutRequestBatchSize = 16;

// Flags for targets are stored in uint32.
constexpr static int32 MaxCopyTargets = 32;

// Buckets distribute keys into shards to reduce contention.
constexpr static uint32 BucketShardCount = 64;

int32 Copy(const TCHAR* Tokens, const TCHAR* Options);

FCommand MakeCopyCommand()
{
	return FCommand(TEXT("Copy"))
		.SetDescription(
			"Copy data from a source cache to one or more target caches.\n\n"
			"-Source specifies a cache graph and is optional when using -Preview.\n"
			"\tExamples:\n"
			"\t-Source=Default\n"
			"\t-Source=(Cloud)\n\n"
			"-Target specifies a cache store and may be specified multiple times.\n"
			"\tExamples:\n"
			"\t-Target=Cloud\n"
			"\t-Target=Cloud(ServerID=Backup)\n"
			"\t-Target=CloudBackup=Cloud(ServerID=Backup)\n"
			"\t-Target=Zen=(Type=Zen,ServerID=Backup)\n\n"
			"-Filter and -FilterRate allow filtering keys from replays. Default inclusion rate is 100%.")
		.SetUsage("-Source=<GraphNameOrConfig> -Target=<StoreNameOrConfig> [-Replay=<Path>]* [-Overwrite] [-Preview]")
		.AddSwitch("Source=<GraphNameOrConfig>", "Source graph name or config.")
		.AddSwitch("Target=<StoreNameOrConfig>", "Target store name or config. Supports multiple targets.")
		//.AddSwitch("Key=<Bucket>/<Hash>", "Key to copy from source to target(s). Supports multiple keys.")
		.AddSwitch("Replay=<Path>", "Replay to copy from source to target(s). Supports multiple replays.")
		.AddSwitch("Filter=<Bucket>[@<Rate>][+...]", "Filter cache keys to copy. Rate is 0-100 percent.")
		.AddSwitch("FilterRate=<Rate>", "Filter rate (0-100) to apply to keys not matched by a bucket or key filter.")
		.AddSwitch("FilterSalt=<Salt>", "Filter salt when filtering cache keys with fractional match rates.")
		.AddSwitch("FilterKeys=<Key>[+...]", "Filter cache keys to copy. Always matches these keys.")
		.AddSwitch("ValueMemoryMB=<Limit>", "Limit memory used to cache values to this amount. Default is 16,384 MiB.")
		.AddSwitch("Preview", "Preview the copy without modifying the target(s).")
		.AddSwitch("Overwrite", "Overwrite data in the target(s) that differs from the replays.")
		.AddSwitch("ForceOverwrite", "Overwrite data in the target(s) without scanning what exists.")
		.OnExecute(Copy);
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

template <CIntegral T>
class TRelaxedAtomicInt
{
public:
	inline TRelaxedAtomicInt(T N) : Value(N) {}
 
	inline operator T() const { return Value.load(std::memory_order_relaxed); }

	inline TRelaxedAtomicInt& operator=(T N) { Value.store(N, std::memory_order_relaxed); return *this; }

	inline void operator+=(T N) { Value.fetch_add(N, std::memory_order_relaxed); }
	inline void operator-=(T N) { Value.fetch_sub(N, std::memory_order_relaxed); }

	inline T operator++() { return Value.fetch_add(1, std::memory_order_relaxed) + 1;}
	inline T operator--() { return Value.fetch_sub(1, std::memory_order_relaxed) - 1; }

private:
	std::atomic<T> Value = 0;
};

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

class FCopyRequestBatch;
class FCopyRequestBatchPool;

struct FCopyRequestBatchApi
{
	// CanDispatch returns the request count when ready to dispatch, otherwise -1.
	// May reserve memory but may not release memory because release can trigger dispatch.
	// Must be trivial because this is called within the pool lock.
	int32 (*CanDispatch)(void*, TConstArrayView<uint32>) = nullptr;
	// Dispatch is called outside of the pool lock after CanDispatch returns a non-negative request count.
	void (*Dispatch)(void*, TConstArrayView<uint32>, IRequestOwner&, const FCopyRequestBatch&) = nullptr;
	// Complete is called outside of the pool lock when the last request in the batch completes.
	void (*Complete)(void*, TConstArrayView<uint32>) = nullptr;
	// Stall is called outside of the pool lock after CanDispatch returns a negative request count.
	void (*Stall)(void*) = nullptr;
	// Context passed to each of the functions.
	void* Context = nullptr;
};

class FCopyRequestBatch
{
public:
	[[nodiscard]] inline bool AddIndex(uint32 Index);

	inline void Complete() const;

private:
	FCopyRequestBatch(const FCopyRequestBatch&) = delete;
	FCopyRequestBatch& operator=(const FCopyRequestBatch&) = delete;

	inline static FCopyRequestBatch* New(TNotNull<FCopyRequestBatchPool*> Pool);
	inline static void Destroy(FCopyRequestBatch* Batch);
	inline explicit FCopyRequestBatch(TNotNull<FCopyRequestBatchPool*> Pool);
	~FCopyRequestBatch() = default;

	TNotNull<FCopyRequestBatchPool*> Pool;
	FCopyRequestBatch* Next = nullptr;
	const FCopyRequestBatchApi* Api = nullptr;
	mutable std::atomic<int32> ActiveCount = 0;
	int32 OwnerIndex = -1;
	int32 IndexCapacity = 0;
	int32 IndexCount = 0;
	uint32 IndexArray[0];

	friend FCopyRequestBatchPool;
};

class FCopyRequestBatchPool
{
public:
	FCopyRequestBatchPool(const FCopyRequestBatchPool&) = delete;
	FCopyRequestBatchPool& operator=(const FCopyRequestBatchPool&) = delete;

	explicit FCopyRequestBatchPool(int32 InBatchSize, int32 InMaxActiveBatchCount, EPriority InPriority)
		: BatchSize(InBatchSize)
		, MaxActiveBatchCount(InMaxActiveBatchCount)
	{
		Owners = (FRequestOwner*)FMemory::Malloc(MaxActiveBatchCount * sizeof(FRequestOwner), alignof(FRequestOwner));
		OwnerFreeList.Reserve(MaxActiveBatchCount);
		for (int32 OwnerIndex = 0; OwnerIndex < MaxActiveBatchCount; ++OwnerIndex)
		{
			new(Owners + OwnerIndex) FRequestOwner(InPriority);
			OwnerFreeList.Add(OwnerIndex);
		}
	}

	~FCopyRequestBatchPool()
	{
		for (FCopyRequestBatch* Batch = Tail; Batch;)
		{
			FCopyRequestBatch* Next = Batch->Next;
			FCopyRequestBatch::Destroy(Batch);
			Batch = Next;
		}

		for (FCopyRequestBatch* Batch = Free; Batch;)
		{
			FCopyRequestBatch* Next = Batch->Next;
			FCopyRequestBatch::Destroy(Batch);
			Batch = Next;
		}

		DestructItems(Owners, MaxActiveBatchCount);
		FMemory::Free(Owners);
	}

	[[nodiscard]] inline bool IsIdle() const
	{
		TUniqueLock Lock(Mutex);
		return ActiveBatchCount == 0 && !Tail;
	}

	[[nodiscard]] inline bool WaitForIdle(FMonotonicTimeSpan WaitTime) const
	{
		FEventCountToken Token = IdleEvent.PrepareWait();
		return IsIdle() || (IdleEvent.WaitFor(Token, WaitTime) && IsIdle());
	}

	[[nodiscard]] inline FCopyRequestBatch* NewBatch(TNotNull<const FCopyRequestBatchApi*> Api)
	{
		FCopyRequestBatch* Batch = nullptr;
		if (TUniqueLock Lock(Mutex); Free)
		{
			Batch = Free;
			Free = Free->Next;
			Batch->Next = nullptr;
		}
		if (!Batch)
		{
			Batch = FCopyRequestBatch::New(this);
		}
		Batch->Api = Api;
		return Batch;
	}

	inline void QueueBatch(FCopyRequestBatch* Batch)
	{
		check(Batch->Pool == this);
		ON_SCOPE_EXIT { TryDispatch(); };
		TUniqueLock Lock(Mutex);
		*Head = Batch;
		Head = &Batch->Next;
	}

	inline void TryDispatch()
	{
		TDynamicUniqueLock Lock(Mutex);
		while (ActiveBatchCount < MaxActiveBatchCount)
		{
			FCopyRequestBatch* Batch = Tail;
			if (!Batch)
			{
				if (ActiveBatchCount == 0)
				{
					IdleEvent.Notify();
				}
				return;
			}

			// Query whether the batch is ready to dispatch.
			const TConstArrayView<uint32> IndexBatch(Batch->IndexArray, Batch->IndexCount);
			const int32 RequestCount = Batch->Api->CanDispatch(Batch->Api->Context, IndexBatch);

			// Pause any further dispatching if the batch is not ready yet.
			if (RequestCount < 0)
			{
				void (*Stall)(void*) = Batch->Api->Stall;
				void* Context = Batch->Api->Context;
				Lock.Unlock();
				Stall(Context);
				return;
			}

			// Remove the batch from the queue.
			Tail = Batch->Next;
			Batch->Next = nullptr;
			if (Head == &Batch->Next)
			{
				Head = &Tail;
			}

			++ActiveBatchCount;

			// Assign an owner from the free list.
			Batch->OwnerIndex = OwnerFreeList.Pop(EAllowShrinking::No);

			Lock.Unlock();

			// Dispatch any requests for the batch.
			Batch->ActiveCount.store(RequestCount, std::memory_order_relaxed);
			Batch->Api->Dispatch(Batch->Api->Context, IndexBatch, Owners[Batch->OwnerIndex], *Batch);

			// Complete the batch immediately if it had no requests to complete.
			if (RequestCount == 0)
			{
				CompleteBatch(Batch);
			}

			Lock.Lock();
		}
	}

private:
	inline void CompleteBatch(FCopyRequestBatch* Batch)
	{
		Batch->Api->Complete(Batch->Api->Context, MakeArrayView(Batch->IndexArray, Batch->IndexCount));

		TUniqueLock Lock(Mutex);

		// Return the owner to the free list.
		OwnerFreeList.Push(Batch->OwnerIndex);
		Batch->OwnerIndex = -1;

		if (--ActiveBatchCount == 0 && !Tail)
		{
			IdleEvent.Notify();
		}

		Batch->Api = nullptr;
		Batch->IndexCount = 0;
		Batch->Next = Free;
		Free = Batch;
	}

	TArray<int32> OwnerFreeList;
	FRequestOwner* Owners = nullptr;

	FCopyRequestBatch* Free = nullptr;
	FCopyRequestBatch* Tail = nullptr;
	FCopyRequestBatch** Head = &Tail;

	const int32 BatchSize = 0;
	const int32 MaxActiveBatchCount = 0;
	int32 ActiveBatchCount = 0;

	mutable FEventCount IdleEvent;
	mutable FMutex Mutex;

	friend FCopyRequestBatch;
};

inline FCopyRequestBatch* FCopyRequestBatch::New(TNotNull<FCopyRequestBatchPool*> InPool)
{
	const SIZE_T Size = sizeof(FCopyRequestBatch) + sizeof(uint32) * InPool->BatchSize;
	return new(FMemory::Malloc(Size, alignof(FCopyRequestBatch))) FCopyRequestBatch(InPool);
}

inline void FCopyRequestBatch::Destroy(FCopyRequestBatch* Batch)
{
	Batch->~FCopyRequestBatch();
	FMemory::Free(Batch);
}

inline FCopyRequestBatch::FCopyRequestBatch(TNotNull<FCopyRequestBatchPool*> InPool)
	: Pool(InPool)
	, IndexCapacity(InPool->BatchSize)
{
	FMemory::Memzero(IndexArray, IndexCapacity * sizeof(uint32));
}

inline bool FCopyRequestBatch::AddIndex(uint32 Index)
{
	IndexArray[IndexCount] = Index;
	return ++IndexCount == IndexCapacity;
}

inline void FCopyRequestBatch::Complete() const
{
	if (ActiveCount.fetch_sub(1, std::memory_order_relaxed) == 1)
	{
		Pool->CompleteBatch(const_cast<FCopyRequestBatch*>(this));
		Pool->TryDispatch();
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

class FCopyRequestQueue
{
public:
	FCopyRequestQueue(const FCopyRequestQueue&) = delete;
	FCopyRequestQueue& operator=(const FCopyRequestQueue&) = delete;

	inline explicit FCopyRequestQueue(TNotNull<FCopyRequestBatchPool*> InPool, TNotNull<const FCopyRequestBatchApi*> InApi)
		: Pool(InPool)
		, Api(InApi)
	{
	}

	inline ~FCopyRequestQueue()
	{
		check(!Batch);
	}

	inline void Queue(uint32 Index)
	{
		FCopyRequestBatch* LocalBatch = nullptr;
		{
			TUniqueLock Lock(Mutex);
			if (!Batch)
			{
				Batch = Pool->NewBatch(Api);
			}
			if (!Batch->AddIndex(Index))
			{
				return;
			}
			LocalBatch = Batch;
			Batch = nullptr;
		}
		Pool->QueueBatch(LocalBatch);
	}

	inline void Dispatch()
	{
		FCopyRequestBatch* LocalBatch = nullptr;
		{
			TUniqueLock Lock(Mutex);
			LocalBatch = Batch;
			Batch = nullptr;
			if (!LocalBatch)
			{
				return;
			}
		}
		Pool->QueueBatch(LocalBatch);
	}

private:
	TNotNull<FCopyRequestBatchPool*> Pool;
	TNotNull<const FCopyRequestBatchApi*> Api;
	FCopyRequestBatch* Batch = nullptr;
	FMutex Mutex;
};

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

struct FCopyParams
{
	FString SourceNameOrConfig;
	TArray<FString> TargetConfigs;
	TArray<FCacheKey> Keys;
	TArray<FString> Replays;
	FCacheKeyFilter Filter;
	uint32 ValueMemoryMB = 16'384;
	bool bDebugSkipPut = false;
	bool bPreview = false;
	bool bOverwrite = false;
	bool bForceOverwrite = false;
};

[[nodiscard]] static bool ParseParams(const TCHAR* Tokens, const TCHAR* Options, FCopyParams& OutParams)
{
	bool bOk = true;
	FString Value;

	if (*Tokens)
	{
		UE_LOGFMT(LogDerivedDataTool, Error, "Unrecognized positional arguments: {Token}", Tokens);
		bOk = false;
	}

	if (const TCHAR* Remaining; FParse::Value(Options, TEXT("-Source="), OutParams.SourceNameOrConfig, /*bShouldStopOnSeparator*/ false, &Remaining))
	{
		if (FParse::Value(Remaining, TEXT("-Source="), Value))
		{
			UE_LOGFMT(LogDerivedDataTool, Error, "More than one '-Source' is not supported. Combine the desired sources into one graph.");
			bOk = false;
		}
	}

	TArray<FString, TInlineAllocator<MaxCopyTargets>> TargetNames;
	for (const TCHAR* Remaining = Options; FParse::Value(Remaining, TEXT("-Target="), Value, /*bShouldStopOnSeparator*/ false, &Remaining);)
	{
		TStringBuilder<256> ProvidedConfig(InPlace, '(', Value, ')');
		Config::FConfigIterator It(ProvidedConfig);
		FStringView StoreName = It.GetName();
		FStringView StoreValue = It.GetValue();
		if (FStringView SplitName, SplitConfig; StoreName.IsEmpty() && Config::TrySplitNameAndConfig(StoreValue, SplitName, SplitConfig))
		{
			StoreName = SplitName;
		}
		if (!StoreName.IsEmpty() && !++It)
		{
			TStringBuilder<256> TargetConfig;
			TargetConfig.AppendChar('(');
			Config::AppendConfig(TargetConfig, StoreName, StoreValue);
			TargetConfig.AppendChar(')');
			const int32 ExpectedNameIndex = TargetNames.Num();
			const int32 NameIndex = TargetNames.AddUnique(FString(StoreName));
			if (NameIndex != ExpectedNameIndex)
			{
				UE_LOGFMT(LogDerivedDataTool, Error, "Target name '{TargetName}' was configured multiple times. Targets must have unique names.", StoreName);
				bOk = false;
			}
			const int32 ExpectedConfigIndex = OutParams.TargetConfigs.Num();
			const int32 ConfigIndex = OutParams.TargetConfigs.AddUnique(FString(TargetConfig));
			if (ConfigIndex != ExpectedConfigIndex)
			{
				UE_LOGFMT(LogDerivedDataTool, Error, "Target '{TargetConfig}' was provided multiple times.", *Value);
				bOk = false;
			}
		}
		else
		{
			UE_LOGFMT(LogDerivedDataTool, Error, "Failed to parse '-Target={TargetConfig}'. Expected a single named cache store. See Help.", *Value);
			bOk = false;
		}
	}
	if (OutParams.TargetConfigs.Num() > MaxCopyTargets)
	{
		UE_LOGFMT(LogDerivedDataTool, Error, "More than {MaxTargetCount} '-Target' are not supported and {TargetCount} were provided. Invoke Copy multiple times.",
			MaxCopyTargets, OutParams.TargetConfigs.Num());
		bOk = false;
	}

	for (const TCHAR* Remaining = Options; FParse::Value(Remaining, TEXT("-Key="), Value, /*bShouldStopOnSeparator*/ false, &Remaining);)
	{
		FCacheKey Key;
		if (TryLexFromString(Key, Value))
		{
			OutParams.Keys.Add(Key);
			UE_LOGFMT(LogDerivedDataTool, Error, "Support for '-Key' is not yet implemented.");
			bOk = false;
		}
		else
		{
			UE_LOGFMT(LogDerivedDataTool, Error, "Failed to parse cache key '{Key}'.", *Value);
			bOk = false;
		}
	}

	for (const TCHAR* Remaining = Options; FParse::Value(Remaining, TEXT("-Replay="), Value, /*bShouldStopOnSeparator*/ false, &Remaining);)
	{
		if (IFileManager::Get().FileExists(*Value))
		{
			OutParams.Replays.Emplace(MoveTemp(Value));
		}
		else
		{
			UE_LOGFMT(LogDerivedDataTool, Error, "Replay file '{ReplayPath}' does not exist.", *Value);
			bOk = false;
		}
	}

	const bool bDefaultMatch = !FParse::Value(Options, TEXT("-Filter="), Value) && !FParse::Value(Options, TEXT("-FilterKeys="), Value);
	float FilterRate = bDefaultMatch ? 100.0f : 0.0f;
	uint32 FilterSalt = 0;
	FParse::Value(Options, TEXT("-FilterRate="), FilterRate);
	FParse::Value(Options, TEXT("-FilterSalt="), FilterSalt);
	OutParams.Filter = FCacheKeyFilter::Parse(Options, TEXT("-Filter="), TEXT("-FilterKeys="), FilterRate);
	OutParams.Filter.SetSalt(FilterSalt);
	UE_CLOGFMT(OutParams.Filter.RequiresSalt(), LogDerivedDataTool, Display,
		"Using salt -FilterSalt={FilterSalt} to filter cache keys.", OutParams.Filter.GetSalt());

	FParse::Value(Options, TEXT("-ValueMemoryMB="), OutParams.ValueMemoryMB);

	OutParams.bDebugSkipPut = FParse::Param(Options, TEXT("-DebugSkipPut"));
	OutParams.bPreview = FParse::Param(Options, TEXT("-Preview"));
	OutParams.bOverwrite = FParse::Param(Options, TEXT("-Overwrite"));
	OutParams.bForceOverwrite = FParse::Param(Options, TEXT("-ForceOverwrite"));

	OutParams.bOverwrite |= OutParams.bForceOverwrite;

	if (OutParams.bForceOverwrite && OutParams.bPreview)
	{
		UE_LOGFMT(LogDerivedDataTool, Error, "Arguments '-ForceOverwrite' and '-Preview' are mutually exclusive.");
		bOk = false;
	}

	if (OutParams.SourceNameOrConfig.IsEmpty() && !OutParams.TargetConfigs.IsEmpty() && (!OutParams.bPreview || !OutParams.Keys.IsEmpty()))
	{
		UE_LOGFMT(LogDerivedDataTool, Error, "Argument '-Source' is required for non-preview copies or when individual keys are provided.");
		bOk = false;
	}

	return bOk;
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

struct FCopyLoadSummary
{
	/** Unique keys for records. */
	TRelaxedAtomicInt<int32> RecordKeyCount = 0;
	/** Unique keys for values. */
	TRelaxedAtomicInt<int32> ValueKeyCount = 0;
	/** Unique key+record count. */
	TRelaxedAtomicInt<int32> RecordCount = 0;
	/** Unique key+record+value count. (Number of values in unique records.) */
	TRelaxedAtomicInt<int32> ValueInRecordCount = 0;
	/** Unique key+value count. */
	TRelaxedAtomicInt<int32> ValueCount = 0;
	/** Total record count before de-duplication. */
	TRelaxedAtomicInt<int32> TotalRecordCount = 0;
	/** Total value count before de-duplication by key+value. */
	TRelaxedAtomicInt<int32> TotalValueCount = 0;

	/** Total raw size of values referenced by unique keys. */
	TRelaxedAtomicInt<uint64> TotalRawSize = 0;
};

inline FCopyLoadSummary& operator+=(FCopyLoadSummary& Out, const FCopyLoadSummary& In)
{
	Out.RecordKeyCount += In.RecordKeyCount;
	Out.ValueKeyCount += In.ValueKeyCount;
	Out.RecordCount += In.RecordCount;
	Out.ValueInRecordCount += In.ValueInRecordCount;
	Out.ValueCount += In.ValueCount;
	Out.TotalRecordCount += In.TotalRecordCount;
	Out.TotalValueCount += In.TotalValueCount;
	Out.TotalRawSize += In.TotalRawSize;
	return Out;
}

struct FCopySummaryValues
{
	int32 KeyCount = 0;
	int32 ValueCount = 0;
	uint64 TotalRawSize = 0;
};

struct FCopySummaryByStatus
{
	FCopySummaryValues Present;
	FCopySummaryValues Differs;
	FCopySummaryValues Missing;
};

struct FCopySummaryByKeyType
{
	FCopySummaryByStatus Record;
	FCopySummaryByStatus Value;
};

inline FCopySummaryValues& operator+=(FCopySummaryValues& Out, const FCopySummaryValues& In)
{
	Out.KeyCount += In.KeyCount;
	Out.ValueCount += In.ValueCount;
	Out.TotalRawSize += In.TotalRawSize;
	return Out;
}

inline FCopySummaryByStatus& operator+=(FCopySummaryByStatus& Out, const FCopySummaryByStatus& In)
{
	Out.Present += In.Present;
	Out.Differs += In.Differs;
	Out.Missing += In.Missing;
	return Out;
}

inline FCopySummaryByKeyType& operator+=(FCopySummaryByKeyType& Out, const FCopySummaryByKeyType& In)
{
	Out.Record += In.Record;
	Out.Value += In.Value;
	return Out;
}

struct FCopyMemoryLimit
{
	uint64 MaxUsedMemory = 0;
	std::atomic<uint64> UsedMemory = 0;
	std::atomic<bool> bReserveFailed = false;
};

struct FCopyProgress
{
	std::atomic<int32> ScanIndex = 0;
	std::atomic<int32> ScanTotal = 0;
	std::atomic<int32> GetIndex = 0;
	std::atomic<int32> GetTotal = 0;
	std::atomic<int32> PutIndex = 0;
	std::atomic<int32> PutTotal = 0;

	std::atomic<FMonotonicTimePoint> LastDisplayTime = {};
};

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

struct FCopyFlags
{
	std::atomic<uint32> TargetDiffersMask = 0; static_assert(MaxCopyTargets <= 8 * sizeof(TargetDiffersMask));
	std::atomic<uint32> TargetMissingMask = 0; static_assert(MaxCopyTargets <= 8 * sizeof(TargetMissingMask));
	std::atomic<uint32> TargetUpdatedMask = 0; static_assert(MaxCopyTargets <= 8 * sizeof(TargetUpdatedMask));
	std::atomic<uint32> SourceMask = 0;

	inline constexpr static uint32 SourceDiffersFlag = 1 << 0;
	inline constexpr static uint32 SourceMissingFlag = 1 << 1;
};

struct FCopyRecord
{
	FSharedString Name;
	FCacheBucket Bucket;
	FIoHash KeyHash;
	uint32 ValueIndex = 0;
	uint32 ValueCount = 0;
	uint32 NextIndex = MAX_uint32;
	FCopyFlags Flags;
	FOptionalCacheRecord Data;
};
static_assert(sizeof(FCopyRecord) == 72); // Padding == 0

struct FCopyValueInRecord
{
	uint64 RawSize = 0;
	FIoHash RawHash;
	FValueId Id;
};
static_assert(sizeof(FCopyValueInRecord) == 40); // Padding == 0

struct FCopyValue
{
	FSharedString Name;
	FCacheBucket Bucket;
	FIoHash KeyHash;
	uint32 NextIndex = MAX_uint32;
	uint64 RawSize = 0;
	FIoHash RawHash;
	FCopyFlags Flags;
	TUniquePtr<FValue> Data;
};
static_assert(sizeof(FCopyValue) == 96); // Padding == 4

struct FCopyBucket
{
	Core::TAlignedElement<FProbingHashTable, PLATFORM_CACHE_LINE_SIZE> RecordKeys[BucketShardCount];
	Core::TAlignedElement<FProbingHashTable, PLATFORM_CACHE_LINE_SIZE> ValueKeys[BucketShardCount];
	TRelaxedAtomicInt<int32> RecordKeyCount = 0;
	TRelaxedAtomicInt<int32> ValueKeyCount = 0;
	FCopyLoadSummary LoadSummary;

	inline static uint32 MapHashToShard(const FIoHash& Hash)
	{
		return reinterpret_cast<const uint32*>(&Hash)[1] % BucketShardCount;
	}
};

struct FCopyContext
{
	alignas(PLATFORM_CACHE_LINE_SIZE) TMap<FCacheBucket, TUniquePtr<FCopyBucket>> Buckets;
	alignas(PLATFORM_CACHE_LINE_SIZE) TConcurrentChunkedSparseArray<FCopyRecord, 32 * 1024> RecordTable;
	alignas(PLATFORM_CACHE_LINE_SIZE) TConcurrentChunkedSparseArray<FCopyValueInRecord, 128 * 1024> ValueInRecordTable;
	alignas(PLATFORM_CACHE_LINE_SIZE) TConcurrentChunkedSparseArray<FCopyValue, 32 * 1024> ValueTable;

	TUniquePtr<ICache> Source;
	TArray<TUniquePtr<ICache>, TFixedAllocator<MaxCopyTargets>> Targets;
	TArray<FStringView, TFixedAllocator<MaxCopyTargets>> TargetNames; // Views into FCopyParams::TargetConfigs

	FCopyRequestBatchApi ScanRecordApi;
	FCopyRequestBatchApi ScanValueApi;
	FCopyRequestBatchPool ScanRequestPool{ScanRequestBatchSize, MaxActiveScanRequestBatchCount, EPriority::High};
	FCopyRequestQueue ScanRecordQueue{&ScanRequestPool, &ScanRecordApi};
	FCopyRequestQueue ScanValueQueue{&ScanRequestPool, &ScanValueApi};

	FCopyRequestBatchApi GetRecordApi;
	FCopyRequestBatchApi GetValueApi;
	FCopyRequestBatchPool GetRequestPool{GetRequestBatchSize, MaxActiveGetRequestBatchCount, EPriority::Normal};
	FCopyRequestQueue GetRecordQueue{&GetRequestPool, &GetRecordApi};
	FCopyRequestQueue GetValueQueue{&GetRequestPool, &GetValueApi};

	FCopyRequestBatchApi PutRecordApi;
	FCopyRequestBatchApi PutValueApi;
	FCopyRequestBatchPool PutRequestPool{PutRequestBatchSize, MaxActivePutRequestBatchCount, EPriority::Normal};
	FCopyRequestQueue PutRecordQueue{&PutRequestPool, &PutRecordApi};
	FCopyRequestQueue PutValueQueue{&PutRequestPool, &PutValueApi};

	FCopyMemoryLimit MemoryLimit;
	FCopyProgress Progress;
	FCopyParams Params;

	[[nodiscard]] uint32 Find(uint32 HeadRecordIndex, TConstArrayView<FValueWithId> Values) const;
	[[nodiscard]] uint32 Find(uint32 HeadValueIndex, const FValue& Value) const;
};

uint32 FCopyContext::Find(uint32 HeadRecordIndex, TConstArrayView<FValueWithId> SearchValues) const
{
	for (uint32 RecordIndex = HeadRecordIndex; RecordIndex != MAX_uint32;)
	{
		const FCopyRecord& Record = RecordTable[RecordIndex];
		if (Record.ValueCount == (uint32)SearchValues.Num())
		{
			bool bEqual = true;
			uint32 ValueIndex = Record.ValueIndex;
			for (const FValueWithId& CacheValue : SearchValues)
			{
				const FCopyValueInRecord& Value = ValueInRecordTable[ValueIndex++];
				bEqual &=
					Value.RawSize == CacheValue.GetRawSize() &&
					Value.RawHash == CacheValue.GetRawHash() &&
					Value.Id == CacheValue.GetId();
			}
			if (bEqual)
			{
				return RecordIndex;
			}
		}
		RecordIndex = Record.NextIndex;
	}
	return MAX_uint32;
}

uint32 FCopyContext::Find(uint32 HeadValueIndex, const FValue& SearchValue) const
{
	for (uint32 ValueIndex = HeadValueIndex; ValueIndex != MAX_uint32;)
	{
		const FCopyValue& Value = ValueTable[ValueIndex];
		if (Value.RawSize == SearchValue.GetRawSize() &&
			Value.RawHash == SearchValue.GetRawHash())
		{
			return ValueIndex;
		}
		ValueIndex = Value.NextIndex;
	}
	return MAX_uint32;
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

inline static TConstArrayView<FValueWithId> GetValuesFromResponse(const FCacheGetResponse& Response)
{
	return Response.Record.GetValues();
}

inline static const FValue& GetValuesFromResponse(const FCacheGetValueResponse& Response)
{
	return Response.Value;
}

inline static bool HasDataInResponse(const FCacheGetResponse& Response)
{
	return Algo::AllOf(Response.Record.GetValues(), &FValue::HasData);
}

inline static bool HasDataInResponse(const FCacheGetValueResponse& Response)
{
	return Response.Value.HasData();
}

inline static void StoreDataFromResponse(FCopyRecord& Record, const FCacheGetResponse& Response)
{
	Record.Data = Response.Record;
}

inline static void StoreDataFromResponse(FCopyValue& Value, const FCacheGetValueResponse& Response)
{
	Value.Data = MakeUnique<FValue>(Response.Value);
}

inline static FCachePutRequest MakePutRequest(const FCopyRecord& Record, ECachePolicy Policy, uint64 UserData)
{
	return {Record.Name, Record.Data.Get(), Policy, UserData};
}

inline static FCachePutValueRequest MakePutRequest(const FCopyValue& Value, ECachePolicy Policy, uint64 UserData)
{
	return {Value.Name, {Value.Bucket, Value.KeyHash}, *Value.Data, Policy, UserData};
}

[[nodiscard]] inline static uint64 MeasureRawSize(const FCopyContext& Context, const FCopyRecord& Record)
{
	uint64 RawSize = 0;
	for (uint32 Index = Record.ValueIndex, End = Record.ValueCount + Index; Index < End; ++Index)
	{
		RawSize += Context.ValueInRecordTable[Index].RawSize;
	}
	return RawSize;
}

[[nodiscard]] inline static uint64 MeasureRawSize(const FCopyContext& Context, const FCopyValue& Value)
{
	return Value.RawSize;
}

[[nodiscard]] inline static uint64 MeasureCompressedSize(const FCopyRecord& Record)
{
	uint64 CompressedSize = 0;
	if (Record.Data)
	{
		for (const FValueWithId& Value : Record.Data.Get().GetValues())
		{
			CompressedSize += Value.GetData().GetCompressedSize();
		}
	}
	return CompressedSize;
}

[[nodiscard]] inline static uint64 MeasureCompressedSize(const FCopyValue& Value)
{
	return Value.Data ? Value.Data->GetData().GetCompressedSize() : 0;
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

class FCacheCopyReplayReader final : public ICacheReplayReader
{
public:
	inline explicit FCacheCopyReplayReader(FCopyContext& InContext)
		: Context(InContext)
		, Filter(Context.Params.Filter)
	{
	}

	void Read(TConstArrayView<FCachePutRequest> Requests, TConstArrayView<FCachePutResponse> Responses, EPriority Priority) final
	{
		int32 Index = 0;
		for (const FCachePutRequest& Request : Requests)
		{
			const FCachePutResponse& Response = Responses[Index++];
			if (Response.Status == EStatus::Ok && Filter && Filter.IsMatch(Response.Record.GetKey()))
			{
				AddRecord(Request.Name, Request.Record.GetKey(), Request.Record.GetValues());
			}
		}
	}

	void Read(TConstArrayView<FCacheGetRequest> Requests, TConstArrayView<FCacheGetResponse> Responses, EPriority Priority) final
	{
		for (const FCacheGetResponse& Response : Responses)
		{
			if (Response.Status == EStatus::Ok && Filter && Filter.IsMatch(Response.Record.GetKey()))
			{
				AddRecord(Response.Name, Response.Record.GetKey(), Response.Record.GetValues());
			}
		}
	}

	void Read(TConstArrayView<FCachePutValueRequest> Requests, TConstArrayView<FCachePutValueResponse> Responses, EPriority Priority) final
	{
		int32 Index = 0;
		for (const FCachePutValueRequest& Request : Requests)
		{
			const FCachePutValueResponse& Response = Responses[Index++];
			if (Response.Status == EStatus::Ok && Filter && Filter.IsMatch(Response.Key))
			{
				AddValue(Request.Name, Request.Key, Request.Value);
			}
		}
	}

	void Read(TConstArrayView<FCacheGetValueRequest> Requests, TConstArrayView<FCacheGetValueResponse> Responses, EPriority Priority) final
	{
		for (const FCacheGetValueResponse& Response : Responses)
		{
			if (Response.Status == EStatus::Ok && Filter && Filter.IsMatch(Response.Key))
			{
				AddValue(Response.Name, Response.Key, Response.Value);
			}
		}
	}

	void Read(TConstArrayView<FCacheGetChunkRequest> Requests, TConstArrayView<FCacheGetChunkResponse> Responses, EPriority Priority) final
	{
		for (const FCacheGetChunkResponse& Response : Responses)
		{
			if (Response.Status == EStatus::Ok && Filter && Filter.IsMatch(Response.Key))
			{
				if (Response.Id.IsNull())
				{
					AddValue(Response.Name, Response.Key, FValue(Response.RawHash, Response.RawSize));
				}
				else
				{
					// Chunk requests from records are ignored because they do not provide enough information about
					// what else is in the record to validate that the expected data is fetched and stored.
				}
			}
		}
	}

private:
	void AddRecord(const FSharedString& Name, const FCacheKey& Key, TConstArrayView<FValueWithId> Values)
	{
		FCopyBucket& Bucket = FindOrCreateBucket(Key.Bucket);

		const uint32 Shard = Bucket.MapHashToShard(Key.Hash);
		TUniqueLock<FMutex> BucketLock(BucketKeysMutex[Shard]);
		FProbingHashTable& RecordKeys = Bucket.RecordKeys[Shard];

		uint32 HeadRecordIndex = MAX_uint32;

		const uint32 KeyHash = GetTypeHash(Key.Hash);
		FProbingHashTable::FSlotId SlotId;
		{
			uint32 Index = 0;
			for (SlotId = RecordKeys.FindFirst(KeyHash, Index); SlotId; SlotId = RecordKeys.FindNext(SlotId, KeyHash, Index))
			{
				if (Context.RecordTable[Index].KeyHash == Key.Hash)
				{
					HeadRecordIndex = Index;
					break;
				}
			}
		}

		++Bucket.LoadSummary.TotalRecordCount;

		// Check for an existing copy of this record.
		if (uint32 ExistingRecordIndex = Context.Find(HeadRecordIndex, Values); ExistingRecordIndex != MAX_uint32)
		{
			FCopyRecord& ExistingRecord = Context.RecordTable[ExistingRecordIndex];
			if (Name < ExistingRecord.Name)
			{
				ExistingRecord.Name = Name;
			}
			return;
		}

		// Add the values to the table.
		const uint32 ValueCount = (uint32)Values.Num();
		const uint32 FirstValueIndex = Context.ValueInRecordTable.AddUninitialized(ValueCount);
		for (uint32 ValueIndex = 0; ValueIndex < ValueCount; ++ValueIndex)
		{
			const FValueWithId& Value = Values.GetData()[ValueIndex];
			++Bucket.LoadSummary.ValueInRecordCount;
			Bucket.LoadSummary.TotalRawSize += Value.GetRawSize();
			new(&Context.ValueInRecordTable[FirstValueIndex + ValueIndex]) FCopyValueInRecord{Value.GetRawSize(), Value.GetRawHash(), Value.GetId()};
		}

		// Add the record to the table.
		++Bucket.LoadSummary.RecordCount;
		if (HeadRecordIndex == MAX_uint32)
		{
			++Bucket.LoadSummary.RecordKeyCount;
			++Bucket.RecordKeyCount;
		}
		const uint32 NewRecordIndex = Context.RecordTable.AddUninitialized();
		new(&Context.RecordTable[NewRecordIndex]) FCopyRecord{Name, Key.Bucket, Key.Hash, FirstValueIndex, ValueCount, HeadRecordIndex};

		// Add or update the index of the head record for this key.
		if (SlotId)
		{
			RecordKeys[SlotId] = NewRecordIndex;
		}
		else
		{
			RecordKeys.Add(KeyHash, NewRecordIndex);
		}
	}

	void AddValue(const FSharedString& Name, const FCacheKey& Key, const FValue& Value)
	{
		FCopyBucket& Bucket = FindOrCreateBucket(Key.Bucket);

		const uint32 Shard = Bucket.MapHashToShard(Key.Hash);
		TUniqueLock<FMutex> BucketLock(BucketKeysMutex[Shard]);
		FProbingHashTable& ValueKeys = Bucket.ValueKeys[Shard];

		uint32 HeadValueIndex = MAX_uint32;

		const uint32 KeyHash = GetTypeHash(Key.Hash);
		FProbingHashTable::FSlotId SlotId;
		{
			uint32 Index = 0;
			for (SlotId = ValueKeys.FindFirst(KeyHash, Index); SlotId; SlotId = ValueKeys.FindNext(SlotId, KeyHash, Index))
			{
				if (Context.ValueTable[Index].KeyHash == Key.Hash)
				{
					HeadValueIndex = Index;
					break;
				}
			}
		}

		++Bucket.LoadSummary.TotalValueCount;

		// Check for an existing copy of this value.
		if (uint32 ExistingValueIndex = Context.Find(HeadValueIndex, Value); ExistingValueIndex != MAX_uint32)
		{
			FCopyValue& ExistingValue = Context.ValueTable[ExistingValueIndex];
			if (Name < ExistingValue.Name)
			{
				ExistingValue.Name = Name;
			}
			return;
		}

		// Add the value to the table.
		++Bucket.LoadSummary.ValueCount;
		if (HeadValueIndex == MAX_uint32)
		{
			++Bucket.LoadSummary.ValueKeyCount;
			++Bucket.ValueKeyCount;
		}
		Bucket.LoadSummary.TotalRawSize += Value.GetRawSize();
		const uint32 NewValueIndex = Context.ValueTable.AddUninitialized();
		new(&Context.ValueTable[NewValueIndex]) FCopyValue{Name, Key.Bucket, Key.Hash, HeadValueIndex, Value.GetRawSize(), Value.GetRawHash()};

		// Add or update the index of the head value for this key.
		if (SlotId)
		{
			ValueKeys[SlotId] = NewValueIndex;
		}
		else
		{
			ValueKeys.Add(KeyHash, NewValueIndex);
		}
	}

	FCopyBucket& FindOrCreateBucket(FCacheBucket BucketName)
	{
		const uint32 BucketNameHash = GetTypeHash(BucketName);

		if (TSharedLock Lock(BucketsMutex); TUniquePtr<FCopyBucket>* Bucket = Context.Buckets.FindByHash(BucketNameHash, BucketName))
		{
			return **Bucket;
		}

		TUniqueLock Lock(BucketsMutex);
		TUniquePtr<FCopyBucket>& Bucket = Context.Buckets.FindOrAddByHash(BucketNameHash, BucketName);
		if (!Bucket)
		{
			Bucket = MakeUnique<FCopyBucket>();
			// Always keep Buckets sorted for display purposes.
			// Bucket count is low which makes this reasonable to do upon creation of each bucket.
			Context.Buckets.KeySort(TLess<>());
		}
		return *Bucket;
	}

	FCopyContext& Context;
	const FCacheKeyFilter Filter;
	alignas(PLATFORM_CACHE_LINE_SIZE) FSharedMutex BucketsMutex;
	Core::TAlignedElement<FMutex, PLATFORM_CACHE_LINE_SIZE> BucketKeysMutex[BucketShardCount];
};

static void DisplayLoadSummary(const FCopyLoadSummary& Summary, FAnsiStringView BucketPrefix)
{
	const int32 RecordVariantCount = Summary.RecordCount - Summary.RecordKeyCount;
	UE_CLOGFMT(Summary.RecordKeyCount > 0, LogDerivedDataTool, Display,
		"{BucketPrefix}Loaded {RecordKeyCount} record keys containing {ValueInRecordCount} values, from {TotalRecordCount} record requests.",
		BucketPrefix, Summary.RecordKeyCount, Summary.ValueInRecordCount, Summary.TotalRecordCount);
	UE_CLOGFMT(RecordVariantCount > 0, LogDerivedDataTool, Warning,
		"{BucketPrefix}Found {RecordVariantCount} records that have the same keys with different values.",
		BucketPrefix, RecordVariantCount);

	const int32 ValueVariantCount = Summary.ValueCount - Summary.ValueKeyCount;
	UE_CLOGFMT(Summary.ValueKeyCount > 0, LogDerivedDataTool, Display,
		"{BucketPrefix}Loaded {ValueKeys} value keys, from {TotalValueCount} value requests.",
		BucketPrefix, Summary.ValueKeyCount, Summary.TotalValueCount);
	UE_CLOGFMT(ValueVariantCount > 0, LogDerivedDataTool, Warning,
		"{BucketPrefix}Found {ValueVariantCount} values that have the same keys with different values.",
		BucketPrefix, ValueVariantCount);

	UE_LOGFMT(LogDerivedDataTool, Display,
		"{BucketPrefix}Referenced values total {TotalRawSize} bytes uncompressed.",
		BucketPrefix, Summary.TotalRawSize);
}

static void DisplayLoadSummary(const FCopyContext& Context)
{
	UE_LOGFMT(LogDerivedDataTool, Display, "---");
	FCopyLoadSummary Summary;
	for (const TPair<FCacheBucket, TUniquePtr<FCopyBucket>>& BucketPair : Context.Buckets)
	{
		const FCacheBucket BucketName = BucketPair.Key;
		const FCopyBucket& Bucket = *BucketPair.Value;
		Summary += Bucket.LoadSummary;
		DisplayLoadSummary(Bucket.LoadSummary, WriteToAnsiString<64>('[', BucketName, ']', ' '));
	}
	UE_LOGFMT(LogDerivedDataTool, Display, "---");
	DisplayLoadSummary(Summary, {});
	UE_LOGFMT(LogDerivedDataTool, Display, "---");
}

[[nodiscard]] static bool LoadReplays(FCopyContext& Context)
{
	const FCopyParams& Params = Context.Params;
	if (Params.Replays.IsEmpty())
	{
		return true;
	}
	UE_LOGF(LogDerivedDataTool, Display, "Loading %d cache replay(s)...", Params.Replays.Num());
	std::atomic<bool> bOk = true;
	FCacheCopyReplayReader Reader(Context);
	const double StartTime = FPlatformTime::Seconds();
	ParallelFor(TEXT("ReadReplays"), Params.Replays.Num(), 1, [&Reader, &Params, &bOk](int32 Index)
	{
		if (!ReadReplayFromFile(Reader, *Params.Replays[Index]))
		{
			UE_LOGF(LogDerivedDataTool, Error, "Failed to read cache replay from '%ls'.", *Params.Replays[Index]);
			bOk.store(false, std::memory_order_relaxed);
		}
	});
	const double EndTime = FPlatformTime::Seconds();
	UE_LOGF(LogDerivedDataTool, Display, "Loaded %d cache replay(s) in %.3fs.", Params.Replays.Num(), EndTime - StartTime);
	DisplayLoadSummary(Context);
	return bOk.load(std::memory_order_relaxed);
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

[[nodiscard]] static bool CreateCopySource(FCopyContext& Context)
{
	UE_LOGF(LogDerivedDataTool, Display, "Creating source cache...");
	if (ICache* Cache = CreateCache(Context.Params.SourceNameOrConfig))
	{
		Context.Source.Reset(Cache);
		return true;
	}
	else
	{
		UE_LOGF(LogDerivedDataTool, Error, "Failed to create target cache from graph config '%ls'.", *Context.Params.SourceNameOrConfig);
		return false;
	}
}

[[nodiscard]] static bool CreateCopyTargets(FCopyContext& Context)
{
	UE_LOGF(LogDerivedDataTool, Display, "Creating %d target caches...", Context.Params.TargetConfigs.Num());
	for (const FString& Config : Context.Params.TargetConfigs)
	{
		if (ICache* Cache = CreateCache(Config))
		{
			Context.Targets.Emplace(Cache);
			Context.TargetNames.Emplace(Config::FConfigIterator(Config).GetName());
		}
		else
		{
			UE_LOGF(LogDerivedDataTool, Error, "Failed to create target cache from graph config '%ls'.", *Config);
		}
	}
	return Context.Targets.Num() == Context.Params.TargetConfigs.Num();
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/** Try to reserve memory. Returns true when reserved. Release with ReleaseMemory. */
[[nodiscard]] static bool TryReserveMemory(FCopyContext& Context, uint64 Memory)
{
	const uint64 MaxUsedMemory = Context.MemoryLimit.MaxUsedMemory;
	for (uint64 UsedMemory = Context.MemoryLimit.UsedMemory.load(std::memory_order_relaxed);;)
	{
		// Allow any size if there is currently no reserved memory,
		// to blocking progress when the required memory for one batch
		// exceeds the memory limit.
		if (UsedMemory + Memory > MaxUsedMemory && UsedMemory > 0)
		{
			Context.MemoryLimit.bReserveFailed.store(true, std::memory_order_release);
			return false;
		}
		// The amount fits within the limit. Try to reserve and retry on failure.
		if (Context.MemoryLimit.UsedMemory.compare_exchange_weak(UsedMemory, UsedMemory + Memory, std::memory_order_relaxed))
		{
			return true;
		}
	}
}

/** Reserve memory without checking the limit. Use only when required and prefer TryReserveMemory. */
static void ReserveMemory(FCopyContext& Context, uint64 Memory)
{
	Context.MemoryLimit.UsedMemory.fetch_add(Memory, std::memory_order_relaxed);
}

/** Release reserved memory and try dispatching from each pool in case they were waiting for memory. */
static void ReleaseMemory(FCopyContext& Context, uint64 Memory)
{
	verify(Context.MemoryLimit.UsedMemory.fetch_sub(Memory, std::memory_order_relaxed) >= Memory);

	if (Context.MemoryLimit.bReserveFailed.load(std::memory_order_relaxed) &&
		Context.MemoryLimit.bReserveFailed.exchange(false, std::memory_order_acq_rel))
	{
		Context.ScanRequestPool.TryDispatch();
		Context.GetRequestPool.TryDispatch();
		Context.PutRequestPool.TryDispatch();
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

template <typename RequestType, typename GetDispatcherType, auto FCopyContext::*TablePtr>
class TCopyScanDispatcher
{
public:
	static FCopyRequestBatchApi GetApi(FCopyContext& Context)
	{
		return {CanDispatch, Dispatch, Complete, [](void*){}, &Context};
	}

private:
	static int32 CanDispatch(void* VoidContext, TConstArrayView<uint32> IndexBatch)
	{
		FCopyContext& Context = *(FCopyContext*)VoidContext;
		return IndexBatch.Num() * Context.Targets.Num();
	}

	static void Dispatch(void* VoidContext, TConstArrayView<uint32> IndexBatch, IRequestOwner& Owner, const FCopyRequestBatch& Batch)
	{
		FCopyContext& Context = *(FCopyContext*)VoidContext;

		TArray<RequestType, TFixedAllocator<ScanRequestBatchSize>> Requests;
		for (uint32 Index : IndexBatch)
		{
			const auto& Request = (Context.*TablePtr)[Index];
			Requests.Add({Request.Name, {Request.Bucket, Request.KeyHash}, ECachePolicy::Query | ECachePolicy::SkipData | ECachePolicy::SkipMeta, uint64(Index)});
		}

		FRequestBarrier Barrier(Owner);
		int32 TargetIndex = 0;
		for (const TUniquePtr<ICache>& Target : Context.Targets)
		{
			for (RequestType& Request : Requests)
			{
				const uint32 Index = uint32(Request.UserData);
				Request.UserData = (uint64(TargetIndex) << 32) | Index;
			}
			((*Target).*CacheFunctionFor<RequestType>)(Requests, Owner, FOnComplete{Context, Batch});
			++TargetIndex;
		}
	}

	static void Complete(void* VoidContext, TConstArrayView<uint32> IndexBatch)
	{
		FCopyContext& Context = *(FCopyContext*)VoidContext;

		Context.Progress.ScanIndex.fetch_add(IndexBatch.Num(), std::memory_order_relaxed);

		if (!Context.Params.bPreview)
		{
			GetDispatcherType::QueueFromScan(Context, IndexBatch);
		}
	}

	struct FOnComplete
	{
		FCopyContext& Context;
		const FCopyRequestBatch& Batch;

		void operator()(TCacheResponseFor<RequestType>&& Response) const
		{
			const int32 TargetIndex = int32(Response.UserData >> 32);
			const uint32 Index = uint32(Response.UserData);

			auto& Request = (Context.*TablePtr)[Index];
			if (Response.Status != EStatus::Ok)
			{
				Request.Flags.TargetMissingMask.fetch_or(uint32(1) << TargetIndex, std::memory_order_relaxed);
				UE_LOGFMT(LogDerivedDataTool, Verbose, "Value in target '{Target}' missing for {Key} from '{Name}'.",
					Context.TargetNames[TargetIndex], GetCacheKey(Response), Response.Name);
			}
			else if (Context.Find(Index, GetValuesFromResponse(Response)) == MAX_uint32)
			{
				Request.Flags.TargetDiffersMask.fetch_or(uint32(1) << TargetIndex, std::memory_order_relaxed);
				UE_LOGFMT(LogDerivedDataTool, Display, "Value in target '{Target}' differs for {Key} from '{Name}'.",
					Context.TargetNames[TargetIndex], GetCacheKey(Response), Response.Name);
			}

			Batch.Complete();
		}
	};
};

template <typename RequestType, typename PutDispatcherType, auto FCopyContext::*TablePtr, auto FCopyContext::*QueuePtr>
class TCopyGetDispatcher
{
public:
	static FCopyRequestBatchApi GetApi(FCopyContext& Context)
	{
		return {CanDispatch, Dispatch, Complete, Stall, &Context};
	}

	static void QueueFromScan(FCopyContext& Context, TConstArrayView<uint32> IndexBatch)
	{
		// Queue every index that has at least one target to copy to.
		for (uint32 Index : IndexBatch)
		{
			auto& Request = (Context.*TablePtr)[Index];
			const uint32 TargetMissingMask = Request.Flags.TargetMissingMask.load(std::memory_order_relaxed);
			const uint32 TargetDiffersMask = Request.Flags.TargetDiffersMask.load(std::memory_order_relaxed);
			if (TargetMissingMask || (TargetDiffersMask && Context.Params.bOverwrite))
			{
				Context.Progress.GetTotal.fetch_add(1, std::memory_order_relaxed);
				Context.Progress.PutTotal.fetch_add(1, std::memory_order_relaxed);
				(Context.*QueuePtr).Queue(Index);
			}
		}
	}

private:
	static int32 CanDispatch(void* VoidContext, TConstArrayView<uint32> IndexBatch)
	{
		FCopyContext& Context = *(FCopyContext*)VoidContext;
		// Calculate the raw size and reserve that until the compressed size is known when the requests complete.
		const auto MeasureIndex = [&Context](uint32 Index) -> uint64 { return MeasureRawSize(Context, (Context.*TablePtr)[Index]); };
		if (!TryReserveMemory(Context, Algo::TransformAccumulate(IndexBatch, MeasureIndex, uint64(0))))
		{
			return -1;
		}
		return IndexBatch.Num();
	}

	static void Dispatch(void* VoidContext, TConstArrayView<uint32> IndexBatch, IRequestOwner& Owner, const FCopyRequestBatch& Batch)
	{
		FCopyContext& Context = *(FCopyContext*)VoidContext;

		TArray<RequestType, TFixedAllocator<GetRequestBatchSize>> Requests;
		for (uint32 Index : IndexBatch)
		{
			const auto& Request = (Context.*TablePtr)[Index];
			Requests.Add({Request.Name, {Request.Bucket, Request.KeyHash}, ECachePolicy::Query, uint64(Index)});
		}

		FRequestBarrier Barrier(Owner);
		((*Context.Source).*CacheFunctionFor<RequestType>)(Requests, Owner, FOnComplete{Context, Batch});
	}

	static void Complete(void* VoidContext, TConstArrayView<uint32> IndexBatch)
	{
		FCopyContext& Context = *(FCopyContext*)VoidContext;

		Context.Progress.GetIndex.fetch_add(IndexBatch.Num(), std::memory_order_relaxed);

		// Calculate the delta between raw and compressed to release/reserve memory now that the size is known.
		const auto MeasureIndexDelta = [&Context](uint32 Index) -> int64
		{
			const auto& Request = (Context.*TablePtr)[Index];
			return int64(MeasureRawSize(Context, Request)) - int64(MeasureCompressedSize(Request));
		};
		const int64 MemoryDelta = Algo::TransformAccumulate(IndexBatch, MeasureIndexDelta, int64(0));
		if (MemoryDelta > 0)
		{
			ReleaseMemory(Context, uint64(MemoryDelta));
		}
		else if (MemoryDelta < 0)
		{
			ReserveMemory(Context, uint64(-MemoryDelta));
		}

		PutDispatcherType::QueueFromGet(Context, IndexBatch);
	}

	static void Stall(void* VoidContext)
	{
		// Dispatch puts when gets have stalled to avoid waiting forever for memory that is tied up in these queues.
		FCopyContext& Context = *(FCopyContext*)VoidContext;
		Context.PutRecordQueue.Dispatch();
		Context.PutValueQueue.Dispatch();
	}

	struct FOnComplete
	{
		FCopyContext& Context;
		const FCopyRequestBatch& Batch;

		void operator()(TCacheResponseFor<RequestType>&& Response) const
		{
			const uint32 Index = uint32(Response.UserData);

			auto& Request = (Context.*TablePtr)[Index];
			if (Response.Status != EStatus::Ok)
			{
				Request.Flags.SourceMask.fetch_or(FCopyFlags::SourceMissingFlag, std::memory_order_relaxed);
				UE_LOGFMT(LogDerivedDataTool, Error, "Value in source missing for {Key} from '{Name}'.",
					GetCacheKey(Response), Response.Name);
			}
			else if (!HasDataInResponse(Response))
			{
				Request.Flags.SourceMask.fetch_or(FCopyFlags::SourceMissingFlag, std::memory_order_relaxed);
				UE_LOGFMT(LogDerivedDataTool, Error, "Value in source missing data for {Key} from '{Name}'.",
					GetCacheKey(Response), Response.Name);
			}
			else if (Context.Find(Index, GetValuesFromResponse(Response)) == MAX_uint32)
			{
				Request.Flags.SourceMask.fetch_or(FCopyFlags::SourceDiffersFlag, std::memory_order_relaxed);
				UE_LOGFMT(LogDerivedDataTool, Error, "Value in source differs for {Key} from '{Name}'.",
					GetCacheKey(Response), Response.Name);
			}
			else
			{
				StoreDataFromResponse(Request, Response);
			}

			Batch.Complete();
		}
	};
};

template <typename RequestType, auto FCopyContext::*TablePtr, auto FCopyContext::*QueuePtr>
class TCopyPutDispatcher
{
public:
	static FCopyRequestBatchApi GetApi(FCopyContext& Context)
	{
		return {CanDispatch, Dispatch, Complete, [](void*){}, &Context};
	}

	static void QueueFromGet(FCopyContext& Context, TConstArrayView<uint32> IndexBatch)
	{
		// Queue every index that fetched data from the source.
		for (uint32 Index : IndexBatch)
		{
			auto& Request = (Context.*TablePtr)[Index];
			const uint32 SourceMask = Request.Flags.SourceMask.load(std::memory_order_relaxed);
			if ((SourceMask & (FCopyFlags::SourceMissingFlag | FCopyFlags::SourceDiffersFlag)) == 0)
			{
				(Context.*QueuePtr).Queue(Index);
			}
			else
			{
				Context.Progress.PutTotal.fetch_sub(1, std::memory_order_relaxed);
			}
		}
	}

private:
	static uint32 CalculateTargetMask(const FCopyContext& Context, uint32 Index)
	{
		const auto& Request = (Context.*TablePtr)[Index];
		const uint32 TargetMissingMask = Request.Flags.TargetMissingMask.load(std::memory_order_relaxed);
		const uint32 TargetDiffersMask = Request.Flags.TargetDiffersMask.load(std::memory_order_relaxed);
		return TargetMissingMask | (TargetDiffersMask & (Context.Params.bOverwrite ? MAX_uint32 : 0));
	}

	static int32 CountPutRequests(const FCopyContext& Context, TConstArrayView<uint32> IndexBatch)
	{
		int32 Count = 0;
		for (uint32 Index : IndexBatch)
		{
			Count += FPlatformMath::CountBits(CalculateTargetMask(Context, Index));
		}
		return Count;
	}

	static int32 CanDispatch(void* VoidContext, TConstArrayView<uint32> IndexBatch)
	{
		FCopyContext& Context = *(FCopyContext*)VoidContext;
		return CountPutRequests(Context, IndexBatch);
	}

	static void Dispatch(void* VoidContext, TConstArrayView<uint32> IndexBatch, IRequestOwner& Owner, const FCopyRequestBatch& Batch)
	{
		FCopyContext& Context = *(FCopyContext*)VoidContext;

		// Calculate which targets to store to from this batch.
		uint32 ActiveTargetMask = 0;
		for (uint32 Index : IndexBatch)
		{
			ActiveTargetMask |= CalculateTargetMask(Context, Index);
		}

		// Dispatch one batch of requests to each target.
		FRequestBarrier Barrier(Owner);
		while (ActiveTargetMask)
		{
			const uint32 NextActiveTargetMask = ActiveTargetMask & (ActiveTargetMask - 1);
			const uint32 TargetMask = NextActiveTargetMask ^ ActiveTargetMask;
			const uint32 TargetIndex = FMath::FloorLog2(TargetMask);
			ActiveTargetMask = NextActiveTargetMask;

			TArray<RequestType, TFixedAllocator<PutRequestBatchSize>> Requests;
			for (uint32 Index : IndexBatch)
			{
				const auto& Request = (Context.*TablePtr)[Index];
				const bool bStoreMissing = !!(Request.Flags.TargetMissingMask.load(std::memory_order_relaxed) & TargetMask);
				const bool bStoreDiffers = !!(Request.Flags.TargetDiffersMask.load(std::memory_order_relaxed) & TargetMask) && Context.Params.bOverwrite;
				if (bStoreMissing | bStoreDiffers)
				{
					const uint64 UserData = (uint64(TargetIndex) << 32) | Index;
					const ECachePolicy Policy = Context.Params.bDebugSkipPut ? ECachePolicy::None : bStoreDiffers ? ECachePolicy::Store : ECachePolicy::Default;
					Requests.Add(MakePutRequest(Request, Policy, UserData));
				}
			}
			((*Context.Targets[TargetIndex]).*CacheFunctionFor<RequestType>)(Requests, Owner, FOnComplete{Context, Batch});
		}
	}

	static void Complete(void* VoidContext, TConstArrayView<uint32> IndexBatch)
	{
		FCopyContext& Context = *(FCopyContext*)VoidContext;

		Context.Progress.PutIndex.fetch_add(IndexBatch.Num(), std::memory_order_relaxed);

		const auto MeasureIndex = [&Context](uint32 Index) -> uint64 { return MeasureCompressedSize((Context.*TablePtr)[Index]); };
		ReleaseMemory(Context, Algo::TransformAccumulate(IndexBatch, MeasureIndex, uint64(0)));

		for (uint32 Index : IndexBatch)
		{
			(Context.*TablePtr)[Index].Data.Reset();
		}
	}

	struct FOnComplete
	{
		FCopyContext& Context;
		const FCopyRequestBatch& Batch;

		void operator()(TCacheResponseFor<RequestType>&& Response) const
		{
			const int32 TargetIndex = int32(Response.UserData >> 32);
			const uint32 Index = uint32(Response.UserData);

			auto& Request = (Context.*TablePtr)[Index];
			if (Response.Status != EStatus::Ok && !Context.Params.bDebugSkipPut)
			{
				UE_LOGFMT(LogDerivedDataTool, Warning, "Failed to store to '{Target}' for {Key} from '{Name}'.",
					Context.TargetNames[TargetIndex], GetCacheKey(Response), Response.Name);
			}
			else
			{
				Request.Flags.TargetUpdatedMask.fetch_or(uint32(1) << TargetIndex, std::memory_order_relaxed);
			}

			Batch.Complete();
		}
	};
};

static void DisplayProgress(FCopyContext& Context)
{
	FCopyProgress& Progress = Context.Progress;

	const int32 ScanIndex = Progress.ScanIndex.load(std::memory_order_relaxed);
	const int32 ScanTotal = Progress.ScanTotal.load(std::memory_order_relaxed);
	const int32 GetIndex = Progress.GetIndex.load(std::memory_order_relaxed);
	const int32 GetTotal = Progress.GetTotal.load(std::memory_order_relaxed);
	const int32 PutIndex = Progress.PutIndex.load(std::memory_order_relaxed);
	const int32 PutTotal = Progress.PutTotal.load(std::memory_order_relaxed);

	static int32 LastScanIndex, LastScanTotal, LastGetIndex, LastGetTotal, LastPutIndex, LastPutTotal;
	if (LastScanIndex == ScanIndex && LastScanTotal == ScanTotal &&
		LastGetIndex == GetIndex && LastGetTotal == GetTotal &&
		LastPutIndex == PutIndex && LastPutTotal == PutTotal)
	{
		return;
	}

	LastScanIndex = ScanIndex;
	LastScanTotal = ScanTotal;
	LastGetIndex = GetIndex;
	LastGetTotal = GetTotal;
	LastPutIndex = PutIndex;
	LastPutTotal = PutTotal;

	Progress.LastDisplayTime.store(FMonotonicTimePoint::Now(), std::memory_order_relaxed);

	const auto AppendProgress = [](FAnsiStringBuilderBase& Out, int32 Index, int32 Total, bool bTotalComplete = true)
	{
		if (Total == 0)
		{
			Out.Append(ANSITEXTVIEW("0/0 (100%)"));
		}
		else if (bTotalComplete)
		{
			Out.Appendf("%d/%d (%d%%)", Index, Total, 100 * Index / Total);
		}
		else
		{
			Out.Appendf("%d/...", Index);
		}
	};

	TAnsiStringBuilder<32> ScanProgress;
	AppendProgress(ScanProgress, ScanIndex, ScanTotal);
	TAnsiStringBuilder<32> GetProgress;
	AppendProgress(GetProgress, GetIndex, GetTotal, ScanIndex == ScanTotal);
	TAnsiStringBuilder<32> PutProgress;
	AppendProgress(PutProgress, PutIndex, PutTotal, ScanIndex == ScanTotal);

	if (Context.Params.bPreview)
	{
		UE_LOGFMT(LogDerivedDataTool, Display, "Scanning {Scan}", ScanProgress);
	}
	else if (Context.Params.bForceOverwrite)
	{
		UE_LOGFMT(LogDerivedDataTool, Display, "Querying {Get} from Source; Storing {Put} to Target(s)",
			GetProgress, PutProgress);
	}
	else
	{
		UE_LOGFMT(LogDerivedDataTool, Display, "Scanning {Scan}; Querying {Get} from Source; Storing {Put} to Target(s)",
			ScanProgress, GetProgress, PutProgress);
	}
}

static void ConditionalDisplayProgress(FCopyContext& Context)
{
	if (!IsInGameThread())
	{
		return;
	}
	const FMonotonicTimePoint Now = FMonotonicTimePoint::Now();
	if (Context.Progress.LastDisplayTime.load(std::memory_order_relaxed) + ProgressPeriod < Now)
	{
		DisplayProgress(Context);
	}
}

using FPutRecordDispatcher = TCopyPutDispatcher<FCachePutRequest, &FCopyContext::RecordTable, &FCopyContext::PutRecordQueue>;
using FPutValueDispatcher = TCopyPutDispatcher<FCachePutValueRequest, &FCopyContext::ValueTable, &FCopyContext::PutValueQueue>;

using FGetRecordDispatcher = TCopyGetDispatcher<FCacheGetRequest, FPutRecordDispatcher, &FCopyContext::RecordTable, &FCopyContext::GetRecordQueue>;
using FGetValueDispatcher = TCopyGetDispatcher<FCacheGetValueRequest, FPutValueDispatcher, &FCopyContext::ValueTable, &FCopyContext::GetValueQueue>;

using FScanRecordDispatcher = TCopyScanDispatcher<FCacheGetRequest, FGetRecordDispatcher, &FCopyContext::RecordTable>;
using FScanValueDispatcher = TCopyScanDispatcher<FCacheGetValueRequest, FGetValueDispatcher, &FCopyContext::ValueTable>;

static void ConfigureApis(FCopyContext& Context)
{
	Context.ScanRecordApi = FScanRecordDispatcher::GetApi(Context);
	Context.ScanValueApi = FScanValueDispatcher::GetApi(Context);

	Context.GetRecordApi = FGetRecordDispatcher::GetApi(Context);
	Context.GetValueApi = FGetValueDispatcher::GetApi(Context);

	Context.PutRecordApi = FPutRecordDispatcher::GetApi(Context);
	Context.PutValueApi = FPutValueDispatcher::GetApi(Context);
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

static FCopySummaryByKeyType CalculateBucketSummary(
	const FCopyContext& Context,
	const FCopyBucket& Bucket,
	TNotNull<TFunctionWithContext<FCopySummaryValues* (const FCopyContext&, const FCopyFlags&, FCopySummaryByStatus&)>> SelectValues)
{
	FCopySummaryByKeyType BucketSummary;
	for (const FProbingHashTable& RecordKeys : Bucket.RecordKeys)
	{
		for (const uint32 Index : RecordKeys)
		{
			const FCopyRecord& Record = Context.RecordTable[Index];
			if (FCopySummaryValues* RecordSummary = SelectValues(Context, Record.Flags, BucketSummary.Record))
			{
				++RecordSummary->KeyCount;
				RecordSummary->ValueCount += Record.ValueCount;
				RecordSummary->TotalRawSize += MeasureRawSize(Context, Record);
			}
		}
	}
	for (const FProbingHashTable& ValueKeys : Bucket.ValueKeys)
	{
		for (const uint32 Index : ValueKeys)
		{
			const FCopyValue& Value = Context.ValueTable[Index];
			if (FCopySummaryValues* ValueSummary = SelectValues(Context, Value.Flags, BucketSummary.Value))
			{
				++ValueSummary->KeyCount;
				++ValueSummary->ValueCount;
				ValueSummary->TotalRawSize += MeasureRawSize(Context, Value);
			}
		}
	}
	return BucketSummary;
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

static void DisplayScanSummary(const FCopySummaryByKeyType& Summary, FAnsiStringView BucketPrefix)
{
	const int32 TotalRecordKeyCount = Summary.Record.Present.KeyCount + Summary.Record.Differs.KeyCount + Summary.Record.Missing.KeyCount;
	UE_CLOGFMT(Summary.Record.Differs.KeyCount > 0, LogDerivedDataTool, Display,
		"{BucketPrefix}Found {KeyCount} record keys ({KeyPct}%) that differ from the replays, referencing {TotalRawSize} bytes in {ValueCount} values.",
		BucketPrefix, Summary.Record.Differs.KeyCount, 100 * Summary.Record.Differs.KeyCount / TotalRecordKeyCount, Summary.Record.Differs.TotalRawSize, Summary.Record.Differs.ValueCount);
	UE_CLOGFMT(Summary.Record.Missing.KeyCount > 0, LogDerivedDataTool, Display,
		"{BucketPrefix}Found {KeyCount} record keys ({KeyPct}%) that are missing in the target, referencing {TotalRawSize} bytes in {ValueCount} values.",
		BucketPrefix, Summary.Record.Missing.KeyCount, 100 * Summary.Record.Missing.KeyCount / TotalRecordKeyCount, Summary.Record.Missing.TotalRawSize, Summary.Record.Missing.ValueCount);

	const int32 TotalValueKeyCount = Summary.Value.Present.KeyCount + Summary.Value.Differs.KeyCount + Summary.Value.Missing.KeyCount;
	UE_CLOGFMT(Summary.Value.Differs.KeyCount > 0, LogDerivedDataTool, Display,
		"{BucketPrefix}Found {KeyCount} value keys ({KeyPct}%) that differ from the replays, referencing {TotalRawSize} bytes.",
		BucketPrefix, Summary.Value.Differs.KeyCount, 100 * Summary.Value.Differs.KeyCount / TotalValueKeyCount, Summary.Value.Differs.TotalRawSize);
	UE_CLOGFMT(Summary.Value.Missing.KeyCount > 0, LogDerivedDataTool, Display,
		"{BucketPrefix}Found {KeyCount} value keys ({KeyPct}%) that are missing in the target, referencing {TotalRawSize} bytes.",
		BucketPrefix, Summary.Value.Missing.KeyCount, 100 * Summary.Value.Missing.KeyCount / TotalValueKeyCount, Summary.Value.Missing.TotalRawSize);

	UE_CLOGFMT(BucketPrefix.IsEmpty() &&
		Summary.Record.Differs.KeyCount == 0 && Summary.Record.Missing.KeyCount == 0 &&
		Summary.Value.Differs.KeyCount == 0 && Summary.Value.Missing.KeyCount == 0,
		LogDerivedDataTool, Display, "Found no keys that differ or are missing in the target.");
}

static void DisplayScanSummary(const FCopyContext& Context)
{
	for (int32 TargetIndex = 0; TargetIndex < Context.TargetNames.Num(); ++TargetIndex)
	{
		UE_LOGFMT(LogDerivedDataTool, Display, "");
		UE_LOGFMT(LogDerivedDataTool, Display, "Summary of Scanning Target '{Target}'", Context.TargetNames[TargetIndex]);
		UE_LOGFMT(LogDerivedDataTool, Display, "---");
		const uint32 TargetMask = uint32(1) << TargetIndex;
		FCopySummaryByKeyType Summary;
		for (const TPair<FCacheBucket, TUniquePtr<FCopyBucket>>& BucketPair : Context.Buckets)
		{
			const FCacheBucket BucketName = BucketPair.Key;
			const FCopyBucket& Bucket = *BucketPair.Value;
			const FCopySummaryByKeyType BucketSummary = CalculateBucketSummary(Context, Bucket,
				[TargetMask](const FCopyContext& Context, const FCopyFlags& Flags, FCopySummaryByStatus& SummaryByStatus) -> FCopySummaryValues*
				{
					return
						(Flags.TargetDiffersMask.load(std::memory_order_relaxed) & TargetMask) ? &SummaryByStatus.Differs :
						(Flags.TargetMissingMask.load(std::memory_order_relaxed) & TargetMask) ? &SummaryByStatus.Missing : &SummaryByStatus.Present;
				});
			Summary += BucketSummary;
			DisplayScanSummary(BucketSummary, WriteToAnsiString<64>('[', BucketName, ']', ' '));
		}
		UE_LOGFMT(LogDerivedDataTool, Display, "---");
		DisplayScanSummary(Summary, {});
		UE_LOGFMT(LogDerivedDataTool, Display, "---");
	}
}

static void ScanTargets(FCopyContext& Context)
{
	// Calculate how many keys need to be scanned.
	for (const TPair<FCacheBucket, TUniquePtr<FCopyBucket>>& BucketPair : Context.Buckets)
	{
		const FCopyBucket& Bucket = *BucketPair.Value;
		Context.Progress.ScanTotal.fetch_add(Bucket.RecordKeyCount + Bucket.ValueKeyCount, std::memory_order_relaxed);
	}

	DisplayProgress(Context);

	// Queue every key to be scanned on every target.
	for (const TPair<FCacheBucket, TUniquePtr<FCopyBucket>>& BucketPair : Context.Buckets)
	{
		const FCacheBucket BucketName = BucketPair.Key;
		const FCopyBucket& Bucket = *BucketPair.Value;
		for (const FProbingHashTable& RecordKeys : Bucket.RecordKeys)
		{
			for (const uint32 Index : RecordKeys)
			{
				Context.ScanRecordQueue.Queue(Index);
			}
		}
		for (const FProbingHashTable& ValueKeys : Bucket.ValueKeys)
		{
			for (const uint32 Index : ValueKeys)
			{
				Context.ScanValueQueue.Queue(Index);
			}
		}
	}

	// Wait for async scan requests.
	Context.ScanRecordQueue.Dispatch();
	Context.ScanValueQueue.Dispatch();
	while (!Context.ScanRequestPool.WaitForIdle(ProgressPeriod))
	{
		ConditionalDisplayProgress(Context);
	}
	DisplayProgress(Context);

	// Summarize the keys that need to be copied.
	DisplayScanSummary(Context);
}

static void QueueForceOverwrite(FCopyContext& Context)
{
	const uint32 TargetMask = uint32((uint64(1) << Context.Targets.Num()) - 1);

	for (const TPair<FCacheBucket, TUniquePtr<FCopyBucket>>& BucketPair : Context.Buckets)
	{
		const FCacheBucket BucketName = BucketPair.Key;
		const FCopyBucket& Bucket = *BucketPair.Value;
		for (const FProbingHashTable& RecordKeys : Bucket.RecordKeys)
		{
			for (const uint32 Index : RecordKeys)
			{
				Context.RecordTable[Index].Flags.TargetDiffersMask.store(TargetMask, std::memory_order_relaxed);
				FGetRecordDispatcher::QueueFromScan(Context, MakeArrayView(&Index, 1));
			}
		}
		for (const FProbingHashTable& ValueKeys : Bucket.ValueKeys)
		{
			for (const uint32 Index : ValueKeys)
			{
				Context.ValueTable[Index].Flags.TargetDiffersMask.store(TargetMask, std::memory_order_relaxed);
				FGetValueDispatcher::QueueFromScan(Context, MakeArrayView(&Index, 1));
			}
		}
	}

	DisplayProgress(Context);
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

static void DisplayGetSummary(const FCopySummaryByKeyType& Summary, FAnsiStringView BucketPrefix)
{
	const int32 TotalRecordKeyCount = Summary.Record.Present.KeyCount + Summary.Record.Differs.KeyCount + Summary.Record.Missing.KeyCount;
	UE_CLOGFMT(Summary.Record.Present.KeyCount > 0, LogDerivedDataTool, Display,
		"{BucketPrefix}Fetched {KeyCount} record keys, referencing {TotalRawSize} bytes in {ValueCount} values.",
		BucketPrefix, Summary.Record.Present.KeyCount, Summary.Record.Present.TotalRawSize, Summary.Record.Present.ValueCount);
	UE_CLOGFMT(Summary.Record.Differs.KeyCount > 0, LogDerivedDataTool, Error,
		"{BucketPrefix}Found {KeyCount} record keys ({KeyPct}%) that differ from the replays, referencing {TotalRawSize} bytes in {ValueCount} values.",
		BucketPrefix, Summary.Record.Differs.KeyCount, 100 * Summary.Record.Differs.KeyCount / TotalRecordKeyCount, Summary.Record.Differs.TotalRawSize, Summary.Record.Differs.ValueCount);
	UE_CLOGFMT(Summary.Record.Missing.KeyCount > 0, LogDerivedDataTool, Error,
		"{BucketPrefix}Found {KeyCount} record keys ({KeyPct}%) that are missing in the source, referencing {TotalRawSize} bytes in {ValueCount} values.",
		BucketPrefix, Summary.Record.Missing.KeyCount, 100 * Summary.Record.Missing.KeyCount / TotalRecordKeyCount, Summary.Record.Missing.TotalRawSize, Summary.Record.Missing.ValueCount);

	const int32 TotalValueKeyCount = Summary.Value.Present.KeyCount + Summary.Value.Differs.KeyCount + Summary.Value.Missing.KeyCount;
	UE_CLOGFMT(Summary.Value.Present.KeyCount > 0, LogDerivedDataTool, Display,
		"{BucketPrefix}Fetched {KeyCount} value keys, referencing {TotalRawSize} bytes.",
		BucketPrefix, Summary.Value.Present.KeyCount, Summary.Value.Present.TotalRawSize);
	UE_CLOGFMT(Summary.Value.Differs.KeyCount > 0, LogDerivedDataTool, Error,
		"{BucketPrefix}Found {KeyCount} value keys ({KeyPct}%) that differ from the replays, referencing {TotalRawSize} bytes.",
		BucketPrefix, Summary.Value.Differs.KeyCount, 100 * Summary.Value.Differs.KeyCount / TotalValueKeyCount, Summary.Value.Differs.TotalRawSize);
	UE_CLOGFMT(Summary.Value.Missing.KeyCount > 0, LogDerivedDataTool, Error,
		"{BucketPrefix}Found {KeyCount} value keys ({KeyPct}%) that are missing in the source, referencing {TotalRawSize} bytes.",
		BucketPrefix, Summary.Value.Missing.KeyCount, 100 * Summary.Value.Missing.KeyCount / TotalValueKeyCount, Summary.Value.Missing.TotalRawSize);
}

static void DisplayGetSummary(const FCopyContext& Context)
{
	UE_LOGFMT(LogDerivedDataTool, Display, "");
	UE_LOGFMT(LogDerivedDataTool, Display, "Summary of Querying from Source");
	UE_LOGFMT(LogDerivedDataTool, Display, "---");
	FCopySummaryByKeyType Summary;
	for (const TPair<FCacheBucket, TUniquePtr<FCopyBucket>>& BucketPair : Context.Buckets)
	{
		const FCacheBucket BucketName = BucketPair.Key;
		const FCopyBucket& Bucket = *BucketPair.Value;
		const FCopySummaryByKeyType BucketSummary = CalculateBucketSummary(Context, Bucket,
			[](const FCopyContext& Context, const FCopyFlags& Flags, FCopySummaryByStatus& SummaryByStatus) -> FCopySummaryValues*
			{
				const uint32 SourceMask = Flags.SourceMask.load(std::memory_order_relaxed);
				const uint32 TargetMissingMask = Flags.TargetMissingMask.load(std::memory_order_relaxed);
				const uint32 TargetDiffersMask = Flags.TargetDiffersMask.load(std::memory_order_relaxed);
				return
					(SourceMask & FCopyFlags::SourceDiffersFlag) ? &SummaryByStatus.Differs :
					(SourceMask & FCopyFlags::SourceMissingFlag) ? &SummaryByStatus.Missing :
					(TargetMissingMask || (TargetDiffersMask && Context.Params.bOverwrite)) ? &SummaryByStatus.Present : nullptr;
			});
		Summary += BucketSummary;
		DisplayGetSummary(BucketSummary, WriteToAnsiString<64>('[', BucketName, ']', ' '));
	}
	UE_LOGFMT(LogDerivedDataTool, Display, "---");
	DisplayGetSummary(Summary, {});
	UE_LOGFMT(LogDerivedDataTool, Display, "---");
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

static void DisplayPutSummary(const FCopySummaryByKeyType& Summary, FAnsiStringView BucketPrefix)
{
	const int32 TotalRecordKeyCount = Summary.Record.Present.KeyCount + Summary.Record.Differs.KeyCount + Summary.Record.Missing.KeyCount;
	UE_CLOGFMT(Summary.Record.Present.KeyCount > 0, LogDerivedDataTool, Display,
		"{BucketPrefix}Stored {KeyCount} record keys, referencing {TotalRawSize} bytes in {ValueCount} values.",
		BucketPrefix, Summary.Record.Present.KeyCount, Summary.Record.Present.TotalRawSize, Summary.Record.Present.ValueCount);
	UE_CLOGFMT(Summary.Record.Differs.KeyCount > 0, LogDerivedDataTool, Warning,
		"{BucketPrefix}Failed to store {KeyCount} record keys ({KeyPct}%) that differ from the replays, referencing {TotalRawSize} bytes in {ValueCount} values.",
		BucketPrefix, Summary.Record.Differs.KeyCount, 100 * Summary.Record.Differs.KeyCount / TotalRecordKeyCount, Summary.Record.Differs.TotalRawSize, Summary.Record.Differs.ValueCount);
	UE_CLOGFMT(Summary.Record.Missing.KeyCount > 0, LogDerivedDataTool, Warning,
		"{BucketPrefix}Failed to store {KeyCount} record keys ({KeyPct}%) that are missing in the target, referencing {TotalRawSize} bytes in {ValueCount} values.",
		BucketPrefix, Summary.Record.Missing.KeyCount, 100 * Summary.Record.Missing.KeyCount / TotalRecordKeyCount, Summary.Record.Missing.TotalRawSize, Summary.Record.Missing.ValueCount);

	const int32 TotalValueKeyCount = Summary.Value.Present.KeyCount + Summary.Value.Differs.KeyCount + Summary.Value.Missing.KeyCount;
	UE_CLOGFMT(Summary.Value.Present.KeyCount > 0, LogDerivedDataTool, Display,
		"{BucketPrefix}Stored {KeyCount} value keys, referencing {TotalRawSize} bytes.",
		BucketPrefix, Summary.Value.Present.KeyCount, Summary.Value.Present.TotalRawSize);
	UE_CLOGFMT(Summary.Value.Differs.KeyCount > 0, LogDerivedDataTool, Warning,
		"{BucketPrefix}Failed to store {KeyCount} value keys ({KeyPct}%) that differ from the replays, referencing {TotalRawSize} bytes.",
		BucketPrefix, Summary.Value.Differs.KeyCount, 100 * Summary.Value.Differs.KeyCount / TotalValueKeyCount, Summary.Value.Differs.TotalRawSize);
	UE_CLOGFMT(Summary.Value.Missing.KeyCount > 0, LogDerivedDataTool, Warning,
		"{BucketPrefix}Failed to store {KeyCount} value keys ({KeyPct}%) that are missing in the target, referencing {TotalRawSize} bytes.",
		BucketPrefix, Summary.Value.Missing.KeyCount, 100 * Summary.Value.Missing.KeyCount / TotalValueKeyCount, Summary.Value.Missing.TotalRawSize);
}

static void DisplayPutSummary(const FCopyContext& Context)
{
	for (int32 TargetIndex = 0; TargetIndex < Context.TargetNames.Num(); ++TargetIndex)
	{
		UE_LOGFMT(LogDerivedDataTool, Display, "");
		UE_LOGFMT(LogDerivedDataTool, Display, "Summary of Storing to Target '{Target}'", Context.TargetNames[TargetIndex]);
		UE_LOGFMT(LogDerivedDataTool, Display, "---");
		const uint32 TargetMask = uint32(1) << TargetIndex;
		FCopySummaryByKeyType Summary;
		for (const TPair<FCacheBucket, TUniquePtr<FCopyBucket>>& BucketPair : Context.Buckets)
		{
			const FCacheBucket BucketName = BucketPair.Key;
			const FCopyBucket& Bucket = *BucketPair.Value;
			const FCopySummaryByKeyType BucketSummary = CalculateBucketSummary(Context, Bucket,
				[TargetMask](const FCopyContext& Context, const FCopyFlags& Flags, FCopySummaryByStatus& SummaryByStatus) -> FCopySummaryValues*
				{
					const bool bStoreUpdated = !!(Flags.TargetUpdatedMask.load(std::memory_order_relaxed) & TargetMask);
					const bool bStoreMissing = !!(Flags.TargetMissingMask.load(std::memory_order_relaxed) & TargetMask);
					const bool bStoreDiffers = !!(Flags.TargetDiffersMask.load(std::memory_order_relaxed) & TargetMask) && Context.Params.bOverwrite;
					return
						bStoreUpdated ? &SummaryByStatus.Present :
						bStoreMissing ? &SummaryByStatus.Missing :
						bStoreDiffers ? &SummaryByStatus.Differs : nullptr;
				});
			Summary += BucketSummary;
			DisplayPutSummary(BucketSummary, WriteToAnsiString<64>('[', BucketName, ']', ' '));
		}
		UE_LOGFMT(LogDerivedDataTool, Display, "---");
		DisplayPutSummary(Summary, {});
		UE_LOGFMT(LogDerivedDataTool, Display, "---");
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

int32 Copy(const TCHAR* Tokens, const TCHAR* Options)
{
	FCopyContext Context;

	if (!ParseParams(Tokens, Options, Context.Params))
	{
		return 1;
	}

	if (!LoadReplays(Context))
	{
		return 1;
	}

	if (Context.Params.TargetConfigs.IsEmpty())
	{
		return 0;
	}

	if (!Context.Params.bPreview || !Context.Params.Keys.IsEmpty())
	{
		if (!CreateCopySource(Context))
		{
			return 1;
		}
	}

	if (!CreateCopyTargets(Context))
	{
		return 1;
	}

	Context.MemoryLimit.MaxUsedMemory = uint64(Context.Params.ValueMemoryMB) * 1'048'576;
	UE_LOGFMT(LogDerivedDataTool, Display, "Limiting memory for copied values to {ValueMemoryMB} MiB.", Context.Params.ValueMemoryMB);

	ConfigureApis(Context);

	if (Context.Params.bForceOverwrite)
	{
		QueueForceOverwrite(Context);
	}
	else
	{
		ScanTargets(Context);
	}

	if (Context.Progress.GetTotal > 0)
	{
		// Wait for async get requests.
		Context.GetRecordQueue.Dispatch();
		Context.GetValueQueue.Dispatch();
		while (!Context.GetRequestPool.WaitForIdle(ProgressPeriod))
		{
			ConditionalDisplayProgress(Context);
		}
		DisplayProgress(Context);
		DisplayGetSummary(Context);

		// Wait for async put requests.
		Context.PutRecordQueue.Dispatch();
		Context.PutValueQueue.Dispatch();
		while (!Context.PutRequestPool.WaitForIdle(ProgressPeriod))
		{
			ConditionalDisplayProgress(Context);
		}
		DisplayProgress(Context);
		DisplayPutSummary(Context);
	}

	return 0;
}

} // UE::DerivedData::Tool
