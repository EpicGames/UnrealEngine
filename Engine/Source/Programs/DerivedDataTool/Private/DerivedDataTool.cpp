// Copyright Epic Games, Inc. All Rights Reserved.

#include "RequiredProgramMainCPPInclude.h"

#include "Algo/Find.h"
#include "Containers/UnrealString.h"
#include "CoreGlobals.h"
#include "DerivedDataCache.h"
#include "DerivedDataToolCommand.h"
#include "HAL/LowLevelMemTracker.h"
#include "Logging/LogMacros.h"
#include "Misc/Parse.h"
#include "Misc/ScopeExit.h"
#include "Misc/StringBuilder.h"
#include "Modules/ModuleManager.h"
#include "ProjectUtilities.h"
#include "Sanitizer/RaceDetector.h"
#include "Templates/UnrealTemplate.h"

DEFINE_LOG_CATEGORY(LogDerivedDataTool);

// These macros are not defined by UBT for an Engine program with bTreatAsEngineModule=true.
#define IMPLEMENT_ENCRYPTION_KEY_REGISTRATION()
#define IMPLEMENT_SIGNING_KEY_REGISTRATION()
IMPLEMENT_APPLICATION(DerivedDataTool, "DerivedDataTool");

namespace UE::DerivedData::Tool
{

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FCommand MakeCopyCommand();
FCommand MakeWorkerCommand();

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

int32 FCommand::Execute(const TCHAR* Command) const
{
	TStringBuilder<1024> Tokens;
	TStringBuilder<1024> Options;

	FString Token;
	for (const TCHAR* End = Command; FParse::Token(End, Token, /*bUseEscape*/ false); Command = End)
	{
		FStringBuilderBase& Out = (**Token == TEXT('-')) ? Options : Tokens;
		Out.Append(Command, UE_PTRDIFF_TO_INT32(End - Command));
	}

	TArray<const FCommand*> Stack;
	return Execute(*Tokens, *Options, Stack);
}

int32 FCommand::Execute(const TCHAR* Tokens, const TCHAR* Options, TArray<const FCommand*>& Stack) const
{
	Stack.Push(this);
	ON_SCOPE_EXIT
	{
		Stack.Pop(EAllowShrinking::No);
	};

	if (FParse::Command(&Tokens, TEXT("Help")))
	{
		return Help(Stack);
	}
	else if (const FCommand* Command = Algo::FindByPredicate(Commands,
		[&Tokens](const FCommand& C) { return FParse::Command(&Tokens, C.Name); }))
	{
		return Command->Execute(Tokens, Options, Stack);
	}
	else if (Function)
	{
		return Function(Tokens, Options);
	}
	else
	{
		return Help(Stack);
	}
}

int32 FCommand::Help(TConstArrayView<const FCommand*> Stack) const
{
	TGuardValue NoTime(GPrintLogTimes, ELogTimes::None);
	TGuardValue NoCategory(GPrintLogCategory, false);
	TGuardValue NoVerbosity(GPrintLogVerbosity, false);

	TUtf8StringBuilder<256> UsageLine;
	UsageLine.Append(ANSITEXTVIEW("Usage: "));
	for (const FCommand* Command : Stack)
	{
		UsageLine.Append(Command->Name).AppendChar(' ');
	}
	const int32 BaseUsageLen = UsageLine.Len();
	UsageLine.Append(Usage);
	UE_LOGF(LogDerivedDataTool, Display, "%s", *UsageLine);

	if (Description)
	{
		UE_LOGF(LogDerivedDataTool, Display, "");
		UE_LOGF(LogDerivedDataTool, Display, "%s", Description);
	}

	if (!Commands.IsEmpty())
	{
		UE_LOGF(LogDerivedDataTool, Display, "");
		UE_LOGF(LogDerivedDataTool, Display, "Commands:");
		UE_LOGF(LogDerivedDataTool, Display, "");
		for (const FCommand& Command : Commands)
		{
			UsageLine.RemoveSuffix(UsageLine.Len() - BaseUsageLen);
			UsageLine.Append(Command.Name).AppendChar(' ').Append(Command.Usage);
			UE_LOGF(LogDerivedDataTool, Display, "    %s", *UsageLine);
		}
	}

	for (const FCommand* Command : Stack)
	{
		if (Command->Switches.IsEmpty())
		{
			continue;
		}
		UE_LOGF(LogDerivedDataTool, Display, "");
		UE_LOGF(LogDerivedDataTool, Display, "Switches:");
		UE_LOGF(LogDerivedDataTool, Display, "");
		for (const FSwitch& Switch : Command->Switches)
		{
			UE_LOGF(LogDerivedDataTool, Display, "    -%-30s    %s", Switch.Name, Switch.Description ? Switch.Description : "");
		}
	}

	GLog->Flush();

	return 1;
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FCommand MakeCreateCacheCommand()
{
	return FCommand(TEXT("CreateCache"))
		.SetDescription("Create the cache graph and exit. Use to validate config or load replays.")
		.AddSwitch("DDC=<GraphNameOrConfig>", "The graph name or config to create the cache graph from.")
		.AddSwitch("DDC-ReplayLoad=<Path>", "Replay to load and replay against the graph. Supports multiple replays.")
		.OnExecute([](auto, auto)
		{
			GetCache();
			return 0;
		});
}

} // UE::DerivedData::Tool

INT32_MAIN_INT32_ARGC_TCHAR_ARGV()
{
	using namespace UE;
	using namespace UE::DerivedData;
	using namespace UE::DerivedData::Tool;

#if USING_INSTRUMENTATION
	Sanitizer::RaceDetector::Initialize();
#endif

	const FTaskTagScope Scope(ETaskTag::EGameThread);

	// Allows this program to accept a project argument on the commandline and use project-specific config
	ProjectUtilities::ParseProjectDirFromCommandline(ArgC, ArgV);

	if (int32 Ret = GEngineLoop.PreInit(ArgC, ArgV))
	{
		return Ret;
	}

	ON_SCOPE_EXIT
	{
		LLM(FLowLevelMemTracker::Get().UpdateStatsPerFrame());
		RequestEngineExit(TEXT("Exiting"));
		FEngineLoop::AppPreExit();
		FModuleManager::Get().UnloadModulesAtShutdown();
		FEngineLoop::AppExit();
	#if USING_INSTRUMENTATION
		Sanitizer::RaceDetector::Shutdown();
	#endif
	};

	const TCHAR* Command = FCommandLine::Get();

	// Skip the optional project argument.
	{
		const TCHAR* RemainingCommand = Command;
		if (FString ProjectToken; FParse::Token(RemainingCommand, ProjectToken, /*bUseEscape*/ true) &&
			ProjectToken.EndsWith(FProjectDescriptor::GetExtension()))
		{
			Command = RemainingCommand;
		}
	}

	FCommand RootCommand = FCommand(TEXT("DerivedDataTool"))
		.SetUsage("[PathToProject.uproject] <Command> ...")
		.AddCommand(MakeCreateCacheCommand())
		.AddCommand(MakeCopyCommand())
		.AddCommand(MakeWorkerCommand())
	;
	return RootCommand.Execute(Command);
}
