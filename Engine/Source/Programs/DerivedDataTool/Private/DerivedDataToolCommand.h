// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Logging/LogCategory.h"

DECLARE_LOG_CATEGORY_EXTERN(LogDerivedDataTool, Log, All);

namespace UE::DerivedData::Tool
{

class FCommand final
{
public:
	using FFunction = int32 (const TCHAR* Tokens, const TCHAR* Options);

	explicit FCommand(const TCHAR* InName)
		: Name(InName)
	{
	}

	constexpr FCommand&& SetUsage(const ANSICHAR* InUsage) &&
	{
		Usage = InUsage;
		return (FCommand&&)*this;
	}

	constexpr FCommand&& SetDescription(const ANSICHAR* InDescription) &&
	{
		Description = InDescription;
		return (FCommand&&)*this;
	}

	FCommand&& AddSwitch(const ANSICHAR* InSwitch, const ANSICHAR* InDescription = nullptr) &&
	{
		Switches.Add({InSwitch, InDescription});
		return (FCommand&&)*this;
	}

	FCommand&& AddCommand(FCommand&& Command) &&
	{
		Commands.Emplace(MoveTemp(Command));
		return (FCommand&&)*this;
	}

	constexpr FCommand&& OnExecute(FFunction* InFunction) &&
	{
		Function = InFunction;
		return (FCommand&&)*this;
	}

	int32 Execute(const TCHAR* Command) const;

private:
	int32 Execute(const TCHAR* Tokens, const TCHAR* Options, TArray<const FCommand*>& Stack) const;

	int32 Help(TConstArrayView<const FCommand*> Stack) const;

	struct FSwitch
	{
		const ANSICHAR* Name = nullptr;
		const ANSICHAR* Description = nullptr;
	};

	const TCHAR* Name = nullptr;
	const ANSICHAR* Usage = nullptr;
	const ANSICHAR* Description = nullptr;
	TArray<FSwitch> Switches;
	TArray<FCommand> Commands;
	FFunction* Function = nullptr;
};

} // UE::DerivedData::Tool
