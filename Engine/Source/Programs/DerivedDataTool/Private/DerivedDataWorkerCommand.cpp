// Copyright Epic Games, Inc. All Rights Reserved.

#include "Compression/CompressedBuffer.h"
#include "Containers/Map.h"
#include "CoreGlobals.h"
#include "DerivedDataBuild.h"
#include "DerivedDataBuildWorker.h"
#include "DerivedDataRequestOwner.h"
#include "DerivedDataToolCommand.h"
#include "HAL/FileManager.h"
#include "Logging/StructuredLog.h"
#include "Misc/Guid.h"
#include "Misc/Optional.h"
#include "Misc/OutputDeviceRedirector.h"
#include "Misc/Parse.h"
#include "TargetReceiptBuildWorker.h"

namespace UE::DerivedData::Tool
{

int32 PackageWorker(const TCHAR* Tokens, const TCHAR* Options);
int32 ExtractWorker(const TCHAR* Tokens, const TCHAR* Options);
int32 DisplayWorker(const TCHAR* Tokens, const TCHAR* Options);

inline static FCommand MakeWorkerPackageCommand()
{
	return FCommand(TEXT("Package"))
		.SetDescription(
			"Package a build worker from a target receipt.\n\n"
			"Default worker path is Saved/BuildWorkers/<Program>-<Platform>.worker.")
		.SetUsage("-Target=<Program.target> [-Worker=<Program.worker>]")
		.AddSwitch("Target=<Program.target>", "Target receipt of the program to package.")
		.AddSwitch("Worker=<Program.worker>", "Package in which to store the build worker.")
		.AddSwitch("ListFiles", "List the files in the build worker package.")
		.AddSwitch("NoCompress", "Package the build worker without compression.")
		.AddSwitch("Symbols", "Package the build worker with its symbol files.")
		.AddSwitch("Force", "Package the build worker even if it is up-to-date.")
		.OnExecute(PackageWorker);
}

inline static FCommand MakeWorkerExtractCommand()
{
	return FCommand(TEXT("Extract"))
		.SetDescription("Extract or copy a build worker to a directory.")
		.SetUsage("-Worker=<Program.worker> -Output=<EmptyDirectory>")
		.AddSwitch("Worker=<Program.worker>", "Package containing the build worker to extract.")
		.AddSwitch("Target=<Program.target>", "Target receipt of the program to copy from directly.")
		.AddSwitch("Output=<Path>", "Directory to extract the worker to. Must not exist yet.")
		.OnExecute(ExtractWorker);
}

inline static FCommand MakeWorkerDisplayCommand()
{
	return FCommand(TEXT("Display"))
		.SetDescription("Display build worker details from a package or target receipt.")
		.SetUsage("-Worker=<Program.worker> *OR* -Target=<Program.target>")
		.AddSwitch("Worker=<Program.worker>", "Package containing the build worker to display.")
		.AddSwitch("Target=<Program.target>", "Target receipt of the program to display as if packaged.")
		.AddSwitch("ListFiles", "List the files in the build worker package.")
		.AddSwitch("NoCompress", "Package the build worker without compression. Only for -Target.")
		.AddSwitch("Symbols", "Package the build worker with its symbol files. Only for -Target.")
		.OnExecute(DisplayWorker);
}

FCommand MakeWorkerCommand()
{
	return FCommand(TEXT("Worker"))
		.SetDescription("Commands to operate on derived data build worker programs.")
		.AddCommand(MakeWorkerPackageCommand())
		.AddCommand(MakeWorkerExtractCommand())
		.AddCommand(MakeWorkerDisplayCommand())
		;
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

struct FParseOptionParams
{
	uint32 MinCount = 0;
	uint32 MaxCount = MAX_uint32;
};

bool ParseOption(const TCHAR* Options, const ANSICHAR* Name, const FParseOptionParams& Params, TFunctionWithContext<void (FStringView Value)> OnOption)
{
	uint32 Count = 0;
	FString Value;
	TStringBuilder<64> Match(InPlace, '-', Name, '=');
	for (const TCHAR* Remaining = Options; FParse::Value(Remaining, *Match, Value, /*bShouldStopOnSeparator*/ false, &Remaining); ++Count)
	{
		OnOption(Value);
	}
	UE_CLOGFMT(Count < Params.MinCount, LogDerivedDataTool, Error, "Less than {Count} '-{Name}' is not supported.", Params.MinCount, Name);
	UE_CLOGFMT(Count > Params.MaxCount, LogDerivedDataTool, Error, "More than {Count} '-{Name}' is not supported.", Params.MaxCount, Name);
	return Count >= Params.MinCount && Count <= Params.MaxCount;
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

struct FDisplaySize
{
	uint64 Size = 0;

	friend void SerializeForLog(FCbWriter& Writer, const FDisplaySize& DisplaySize)
	{
		TUtf8StringBuilder<64> Text;
		if (DisplaySize.Size < uint64(10) * 1024)
		{
			Text << (DisplaySize.Size) << " B";
		}
		else if (DisplaySize.Size < uint64(10) * 1024 * 1024)
		{
			Text << (DisplaySize.Size / 1024) << " KiB";
		}
		else if (DisplaySize.Size < uint64(10) * 1024 * 1024 * 1024)
		{
			Text << (DisplaySize.Size / 1024 / 1024) << " MiB";
		}
		else
		{
			Text << (DisplaySize.Size / 1024 / 1024 / 1024) << " GiB";
		}

		Writer.BeginObject();
		Writer.AddInteger("Size", DisplaySize.Size);
		Writer.AddString("$text", Text);
		Writer.EndObject();
	}
};

FDisplaySize DisplaySize(uint64 Size)
{
	return FDisplaySize{Size};
}

void DisplayWorker(const FBuildWorker& Worker, bool bListFiles)
{
	TGuardValue NoTime(GPrintLogTimes, ELogTimes::None);
	TGuardValue NoCategory(GPrintLogCategory, false);
	TGuardValue NoVerbosity(GPrintLogVerbosity, false);

	uint64 TotalFiles = 0;
	uint64 TotalRawSize = 0;
	TArray<FIoHash> RawHashes;
	const auto FileVisitor = [&TotalFiles, &TotalRawSize, &RawHashes](FUtf8StringView Path, const FIoHash& RawHash, uint64 RawSize)
	{
		++TotalFiles;
		TotalRawSize += RawSize;
		RawHashes.Add(RawHash);
	};
	Worker.IterateFiles(FileVisitor);
	Worker.IterateExecutables(FileVisitor);
	int64 TotalCompressedSize = IFileManager::Get().FileSize(*WriteToString<256>(Worker.GetPackagePath()));

	UE_LOGFMT(LogDerivedDataTool, Display, "");
	UE_CLOGFMT(!Worker.GetPackagePath().IsEmpty(),
		LogDerivedDataTool, Display, "Build Worker Package: {PackagePath}", Worker.GetPackagePath());
	UE_LOGFMT(LogDerivedDataTool, Display, "Build Worker Name: {WorkerName}", Worker.GetName());
	UE_LOGFMT(LogDerivedDataTool, Display, "Build Worker Host: {HostPlatform}", Worker.GetHostPlatform());
	UE_LOGFMT(LogDerivedDataTool, Display, "Build Worker Path: {WorkerPath}", Worker.GetPath());
	UE_LOGFMT(LogDerivedDataTool, Display, "Build Worker Key: {WorkerKey}", Worker.GetKey().Hash);
	UE_LOGFMT(LogDerivedDataTool, Display, "Build System Version: {BuildSystemVersion}", Worker.GetBuildSystemVersion());
	UE_LOGFMT(LogDerivedDataTool, Display,
		"Build Worker Summary: {FileCount} files, {RawSize} uncompressed, {CompressedSize} compressed",
		TotalFiles, DisplaySize(TotalRawSize), DisplaySize(TotalCompressedSize));
	UE_LOGFMT(LogDerivedDataTool, Display, "");

	bool bHasFunctions = false;
	Worker.IterateFunctions([&bHasFunctions](FUtf8StringView Name, const FGuid& Version)
	{
		UE_LOGFMT(LogDerivedDataTool, Display, "Function {Name} @ {Version}", Name, Version);
		bHasFunctions = true;
	});
	UE_CLOGFMT(bHasFunctions, LogDerivedDataTool, Display, "");

	bool bHasEnvironment = false;
	Worker.IterateEnvironment([&bHasEnvironment](FUtf8StringView Name, FUtf8StringView Value)
	{
		UE_LOGFMT(LogDerivedDataTool, Display, "Environment {Name}={Value}", Name, Value);
		bHasEnvironment = true;
	});
	UE_CLOGFMT(bHasEnvironment, LogDerivedDataTool, Display, "");

	if (bListFiles)
	{
		bool bHasDirectory = false;
		Worker.IterateDirectories([&bHasDirectory](FUtf8StringView Path)
		{
			UE_LOGFMT(LogDerivedDataTool, Display, "Directory {Path}", Path);
			bHasDirectory = true;
		});
		UE_CLOGFMT(bHasDirectory, LogDerivedDataTool, Display, "");

		TMap<FIoHash, uint64> RawHashToCompressedSize;
		FRequestOwner Owner(EPriority::Blocking);
		Worker.Resolve(RawHashes, Owner, [&RawHashToCompressedSize](FBuildWorkerResolvedParams&& Params)
		{
			for (const FCompressedBuffer& File : Params.Files)
			{
				RawHashToCompressedSize.Add(File.GetRawHash(), File.GetCompressedSize());
			}
		});
		Owner.Wait();

		const auto ListVisitor = [&RawHashToCompressedSize](FUtf8StringView Path, const FIoHash& RawHash, uint64 RawSize)
		{
			UE_LOGFMT(LogDerivedDataTool, Display,
				"File {Path}: {RawSize} uncompressed, {CompressedSize} compressed, {RawHash}",
				Path, DisplaySize(RawSize), DisplaySize(RawHashToCompressedSize.FindRef(RawHash, 0)), RawHash);
		};
		Worker.IterateExecutables(ListVisitor);
		Worker.IterateFiles(ListVisitor);
		UE_LOGFMT(LogDerivedDataTool, Display, "");
	}

	GLog->Flush();
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

int32 PackageWorker(const TCHAR* Tokens, const TCHAR* Options)
{
	FString TargetPath;
	FString WorkerPath;
	if (!ParseOption(Options, "Target", {.MinCount = 1, .MaxCount = 1}, [&TargetPath](FStringView Value) { TargetPath = Value; }) ||
		!ParseOption(Options, "Worker", {.MaxCount = 1}, [&WorkerPath](FStringView Value) { WorkerPath = Value; }))
	{
		return 1;
	}

	const bool bListFiles = FParse::Param(Options, TEXT("-ListFiles"));
	const bool bNoCompress = FParse::Param(Options, TEXT("-NoCompress"));
	const bool bSymbols = FParse::Param(Options, TEXT("-Symbols"));
	const bool bForce = FParse::Param(Options, TEXT("-Force"));

	const FTargetReceiptBuildWorkerParams Params
	{
		.WorkerPackagePath = WorkerPath,
		.bAlwaysBuild = bForce,
		.bIncludeSymbols = bSymbols,
		.bSkipCompression = bNoCompress,
	};

	if (TOptional<FBuildWorker> Worker = CreateBuildWorkerFromTargetReceipt(TargetPath, GetBuild().GetWorkerRegistry(), Params))
	{
		DisplayWorker(*Worker, bListFiles);
		return 0;
	}
	else
	{
		UE_LOGFMT(LogDerivedDataTool, Error, "Failed to package build worker. See log for details.");
		return 1;
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

int32 ExtractWorker(const TCHAR* Tokens, const TCHAR* Options)
{
	FString WorkerPath;
	FString TargetPath;
	FString OutputPath;
	if (!ParseOption(Options, "Worker", {.MaxCount = 1}, [&WorkerPath](FStringView Value) { WorkerPath = Value; }) ||
		!ParseOption(Options, "Target", {.MaxCount = 1}, [&TargetPath](FStringView Value) { TargetPath = Value; }) ||
		!ParseOption(Options, "Output", {.MinCount = 1, .MaxCount = 1}, [&OutputPath](FStringView Value) { OutputPath = Value; }))
	{
		return 1;
	}

	if (!WorkerPath.IsEmpty() && !TargetPath.IsEmpty())
	{
		UE_LOGFMT(LogDerivedDataTool, Error, "Only one of '-Worker' and '-Target' is supported.");
		return 1;
	}

	if (IFileManager::Get().DirectoryExists(*OutputPath))
	{
		UE_LOGFMT(LogDerivedDataTool, Error, "Output directory '{OutputPath}' must not exist yet.", OutputPath);
		return 1;
	}

	TOptional<FBuildWorker> Worker;
	if (!WorkerPath.IsEmpty())
	{
		Worker = FBuildWorker::Load(WorkerPath);
	}
	else
	{
		Worker = CreateBuildWorkerFromTargetReceipt(TargetPath, GetBuild().GetWorkerRegistry());
	}

	if (!Worker)
	{
		UE_LOGFMT(LogDerivedDataTool, Error, "Failed to load build worker package. See log for details and confirm that the worker/target exists.");
		return 1;
	}

	if (Worker->Extract(OutputPath))
	{
		UE_LOGFMT(LogDerivedDataTool, Display, "Extracted build worker to '{OutputPath}'.", OutputPath);
		return 0;
	}
	else
	{
		UE_LOGFMT(LogDerivedDataTool, Error, "Failed to extract build worker. See log for details.");
		return 1;
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

int32 DisplayWorker(const TCHAR* Tokens, const TCHAR* Options)
{
	TArray<FString> WorkerPaths;
	TArray<FString> TargetPaths;
	if (!ParseOption(Options, "Worker", {}, [&WorkerPaths](FStringView Value) { WorkerPaths.Emplace(Value); }) ||
		!ParseOption(Options, "Target", {}, [&TargetPaths](FStringView Value) { TargetPaths.Emplace(Value); }))
	{
		return 1;
	}

	const bool bListFiles = FParse::Param(Options, TEXT("-ListFiles"));
	const bool bNoCompress = FParse::Param(Options, TEXT("-NoCompress"));
	const bool bSymbols = FParse::Param(Options, TEXT("-Symbols"));
	const bool bForce = FParse::Param(Options, TEXT("-Force"));

	for (const FString& WorkerPath : WorkerPaths)
	{
		if (TOptional<FBuildWorker> Worker = FBuildWorker::Load(WorkerPath))
		{
			DisplayWorker(*Worker, bListFiles);
		}
		else if (!IFileManager::Get().FileExists(*WorkerPath))
		{
			UE_LOGFMT(LogDerivedDataTool, Error, "Failed to load build worker package from '{WorkerPath}' because the file does not exist.", WorkerPath);
		}
		else
		{
			UE_LOGFMT(LogDerivedDataTool, Error, "Failed to load build worker package from '{WorkerPath}'. See log for details.", WorkerPath);
		}
	}

	for (const FString& TargetPath : TargetPaths)
	{
		const FTargetReceiptBuildWorkerParams Params{.bIncludeSymbols = bSymbols, .bSkipCompression = bNoCompress};
		if (TOptional<FBuildWorker> Worker = CreateBuildWorkerFromTargetReceipt(TargetPath, GetBuild().GetWorkerRegistry(), Params))
		{
			DisplayWorker(*Worker, bListFiles);
		}
		else
		{
			UE_LOGFMT(LogDerivedDataTool, Error, "Failed to package build worker from target receipt '{TargetPath}'. See log for details.", TargetPath);
		}
	}

	return 0;
}

} // UE::DerivedData::Tool
